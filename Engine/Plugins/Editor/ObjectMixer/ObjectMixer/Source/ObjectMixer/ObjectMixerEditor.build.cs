// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class ObjectMixerEditor : ModuleRules
{
	public ObjectMixerEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"OutputLog",
				"PropertyEditor", 
				"SceneOutliner"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"PlacementMode",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"AdvancedWidgets",
				"ApplicationCore",
				"AssetRegistry",
				"AssetTools",
				"CoreUObject",
				"ContentBrowser",
				"DeveloperSettings",
				"Engine",
				"EditorConfig",
				"EditorStyle",
				"EditorWidgets",
				"EngineAssetDefinitions",
				"InputCore",
				"Kismet",
				"LevelEditor",
				"LevelSequence",
				"Projects",
				"Sequencer",
				"Slate",
				"SlateCore",
				"ToolMenus", 
				"ToolWidgets",
				"TypedElementFramework",
				"TypedElementRuntime",
				"UMG",
				"UnrealEd",
				"WorkspaceMenuStructure",
				"SourceControl",
			}
		);
	}
}
