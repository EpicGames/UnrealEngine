// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"
#include "Script/AssetDefinition_Blueprint.h"
#include "AssetDefinition_MixerBlueprintObjectFilter.generated.h"

UCLASS()
class UAssetDefinition_MixerBlueprintObjectFilter : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_MixerBlueprintObjectFilter", "Object Mixer Filter"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UObjectMixerBlueprintObjectFilter::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Misc, NSLOCTEXT("AssetDefinition", "AssetDefinition_MixerBlueprintObjectFilterSubMenu", "Other"), ECategoryMenuType::Section)
			};
		return Categories;
	}
	// UAssetDefinition End
};
