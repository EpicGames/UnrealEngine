// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"

#include "ObjectMixerEditorModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectMixerEditorObjectFilter)

void UObjectMixerObjectFilter::PostCDOCompiled(const FPostCDOCompiledContext& Context)
{
	UObject::PostCDOCompiled(Context);

	// Only call on in-editor compile
	if (!Context.bIsRegeneratingOnLoad)
	{
		FObjectMixerEditorModule::Get().OnBlueprintFilterCompiled().Broadcast();
	}
}

bool UObjectMixerObjectFilter::GetShowTransientObjects() const
{
	return false;
}

bool UObjectMixerObjectFilter::ShouldAllowHybridRows() const
{
	return true;
}

bool UObjectMixerObjectFilter::ShouldAllowColumnCustomizationByUser() const
{
	return true;
}

TSet<FName> UObjectMixerObjectFilter::GetColumnsToShowByDefault() const
{
	return {};
}

TSet<FName> UObjectMixerObjectFilter::GetColumnsToExclude() const
{
	return {};
}

EObjectMixerInheritanceInclusionOptions UObjectMixerObjectFilter::GetObjectMixerPropertyInheritanceInclusionOptions() const
{
	return EObjectMixerInheritanceInclusionOptions::None;
}

EObjectMixerInheritanceInclusionOptions UObjectMixerObjectFilter::GetObjectMixerPlacementClassInclusionOptions() const
{
	return EObjectMixerInheritanceInclusionOptions::None;
}

bool UObjectMixerObjectFilter::ShouldIncludeUnsupportedProperties() const
{
	return false;
}

TSet<FName> UObjectMixerObjectFilter::GetPropertiesThatRequireListRefresh() const
{
	return { "Mobility" };
}

TArray<AActor*> UObjectMixerObjectFilter::FindAssociatedActors(AActor* InActor) const
{
	return {};
}

bool UObjectMixerObjectFilter::IsActorAssociated(AActor* Actor, AActor* AssociatedActor) const
{
	return false;
}

TArray<FObjectMixerEditorActorSubObject> UObjectMixerObjectFilter::GetActorSubObjects(AActor* InActor) const
{
	return {};
}

bool UObjectMixerObjectFilter::HasCustomDropHandling(const ISceneOutlinerTreeItem& DropTarget) const
{
	return false;
}

FSceneOutlinerDragValidationInfo UObjectMixerObjectFilter::ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const
{
	return FSceneOutlinerDragValidationInfo::Invalid();
}

void UObjectMixerObjectFilter::OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const
{
}

TSet<UClass*> UObjectMixerObjectFilter::GetParentAndChildClassesFromSpecifiedClasses(
	const TSet<UClass*>& InSpecifiedClasses, EObjectMixerInheritanceInclusionOptions Options)
{
	// 'None' means we only want the specified classes
	if (Options == EObjectMixerInheritanceInclusionOptions::None)
	{
		return InSpecifiedClasses;
	}
	
	TSet<UClass*> ReturnValue;

	auto GetChildClassesLambda = [](const UClass* Class, TSet<UClass*>& OutReturnValue, const bool bRecursive)
	{
		TArray<UClass*> DerivedClasses;
		GetDerivedClasses(Class, DerivedClasses, bRecursive);
		OutReturnValue.Append(DerivedClasses);
	};

	for (UClass* Class : InSpecifiedClasses)
	{
		ReturnValue.Add(Class);

		// Super Classes

			// Immediate only
		if (Options == EObjectMixerInheritanceInclusionOptions::IncludeOnlyImmediateParent ||
			Options == EObjectMixerInheritanceInclusionOptions::IncludeOnlyImmediateParentAndChildren ||
			Options == EObjectMixerInheritanceInclusionOptions::IncludeOnlyImmediateParentAndAllChildren)
		{
			if (UClass* Super = Class->GetSuperClass())
			{
				ReturnValue.Add(Super);
			}
		}
			// All Parents
		if (Options == EObjectMixerInheritanceInclusionOptions::IncludeAllParents ||
			Options == EObjectMixerInheritanceInclusionOptions::IncludeAllParentsAndChildren ||
			Options == EObjectMixerInheritanceInclusionOptions::IncludeAllParentsAndOnlyImmediateChildren)
		{
			if (UClass* Super = Class->GetSuperClass())
			{
				while (Super)
				{
					ReturnValue.Add(Super);
					Super = Super->GetSuperClass();
				}
			}
		}

		// Child Classes

			// Immediate only
		if (Options == EObjectMixerInheritanceInclusionOptions::IncludeOnlyImmediateChildren ||
			Options == EObjectMixerInheritanceInclusionOptions::IncludeOnlyImmediateParentAndChildren ||
			Options == EObjectMixerInheritanceInclusionOptions::IncludeAllParentsAndOnlyImmediateChildren)
		{
			GetChildClassesLambda(Class, ReturnValue, false);
		}
			// All Children
		if (Options == EObjectMixerInheritanceInclusionOptions::IncludeAllChildren ||
			Options == EObjectMixerInheritanceInclusionOptions::IncludeAllParentsAndChildren ||
			Options == EObjectMixerInheritanceInclusionOptions::IncludeOnlyImmediateParentAndAllChildren)
		{
			GetChildClassesLambda(Class, ReturnValue, true);
		}
	}

	return ReturnValue;
}

TSet<UClass*> UObjectMixerObjectFilter::GetParentAndChildClassesFromSpecifiedClasses(
	const TSet<TSubclassOf<AActor>>& InSpecifiedClasses, EObjectMixerInheritanceInclusionOptions Options)
{
	TSet<UClass*> AsUClasses;
	
	for (TSubclassOf<AActor> ActorClass : InSpecifiedClasses)
	{
		AsUClasses.Add(ActorClass);
	}

	return GetParentAndChildClassesFromSpecifiedClasses(AsUClasses, Options);
}
