// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectMixerEditorModule.h"

#include "ObjectMixerEditorLog.h"
#include "ObjectMixerEditorSettings.h"
#include "ObjectMixerEditorSerializedData.h"
#include "ObjectMixerEditorStyle.h"
#include "Views/List/ObjectMixerEditorList.h"
#include "Views/Widgets/ObjectMixerEditorListMenuContext.h"

#include "Engine/Level.h"
#include "ISettingsModule.h"
#include "LevelEditorSequencerIntegration.h"
#include "Misc/CoreDelegates.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

const FName FObjectMixerEditorModule::BaseObjectMixerModuleName("ObjectMixerEditor");

#define LOCTEXT_NAMESPACE "ObjectMixerEditor"

IMPLEMENT_MODULE(FObjectMixerEditorModule, ObjectMixerEditor)

void FObjectMixerEditorModule::StartupModule()
{
	FObjectMixerEditorStyle::Initialize();

	// In the future, Object Mixer and Light Mixer tabs may go into an Object Mixer group
	//RegisterMenuGroup();
	
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FObjectMixerEditorModule::Initialize);
}

void FObjectMixerEditorModule::ShutdownModule()
{
	FObjectMixerEditorStyle::Shutdown();

	UnregisterMenuGroup();
	
	Teardown();
}

UWorld* FObjectMixerEditorModule::GetWorld()
{
	check(GEditor);
	return GEditor->GetEditorWorldContext().World();
}

void FObjectMixerEditorModule::Initialize()
{	
	SetupMenuItemVariables();
	
	RegisterTabSpawner();
	RegisterSettings();
}

void FObjectMixerEditorModule::Teardown()
{
	ListModel.Reset();

	UToolMenus::UnregisterOwner(this);
	
	UnregisterTabSpawner();
	UnregisterSettings();
}

FObjectMixerEditorModule& FObjectMixerEditorModule::Get()
{
	return FModuleManager::LoadModuleChecked<FObjectMixerEditorModule>("ObjectMixerEditor");
}

void FObjectMixerEditorModule::OpenProjectSettings()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings")
		.ShowViewer("Editor", "Plugins", "Object Mixer");
}

FName FObjectMixerEditorModule::GetModuleName() const
{
	return "ObjectMixerEditor";
}

TSharedPtr<SWidget> FObjectMixerEditorModule::MakeObjectMixerDialog(
	TSubclassOf<UObjectMixerObjectFilter> InDefaultFilterClass)
{
	if (!ListModel.IsValid())
	{
		ListModel = MakeShared<FObjectMixerEditorList>(GetModuleName());
		ListModel->Initialize();
	}

	if (InDefaultFilterClass)
	{
		ListModel->SetDefaultFilterClass(InDefaultFilterClass);
	}
	
	const TSharedPtr<SWidget> ObjectMixerDialog = ListModel->CreateWidget();

	return ObjectMixerDialog;
}

TArray<TWeakPtr<ISequencer>> FObjectMixerEditorModule::GetSequencers() const
{
	return FLevelEditorSequencerIntegration::Get().GetSequencers();
}

TSharedPtr<SDockTab> FObjectMixerEditorModule::FindNomadTab()
{
	if (!DockTab.IsValid())
	{
		DockTab = FGlobalTabmanager::Get()->FindExistingLiveTab(GetTabSpawnerId());
	}
	
	return DockTab.Pin();
}

bool FObjectMixerEditorModule::RegenerateListWidget()
{
	if (const TSharedPtr<SDockTab> FoundTab = FindNomadTab())
	{
		const TSharedPtr<SWidget> ObjectMixerDialog = MakeObjectMixerDialog(DefaultFilterClass);
		FoundTab->SetContent(ObjectMixerDialog ? ObjectMixerDialog.ToSharedRef() : SNullWidget::NullWidget);
		return true;
	}

	return false;
}

void FObjectMixerEditorModule::OnRenameCommand()
{
	if (ListModel.IsValid())
	{
		ListModel->OnRenameCommand();
	}
}

void FObjectMixerEditorModule::RegisterMenuGroup()
{
	WorkspaceGroup = WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory()->AddGroup(
		LOCTEXT("ObjectMixerMenuGroupItemName", "Object Mixer"), 
		FSlateIcon(FObjectMixerEditorStyle::Get().GetStyleSetName(),
			"ObjectMixer.ToolbarButton", "ObjectMixer.ToolbarButton.Small"));
}

void FObjectMixerEditorModule::UnregisterMenuGroup()
{
	if (WorkspaceGroup)
	{
		for (const TSharedRef<FWorkspaceItem>& ChildItem : WorkspaceGroup->GetChildItems())
		{
			WorkspaceGroup->RemoveItem(ChildItem);
		}
		
		WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory()->RemoveItem(WorkspaceGroup.ToSharedRef());
		WorkspaceGroup.Reset();
	}
}

void FObjectMixerEditorModule::SetupMenuItemVariables()
{
	TabLabel = LOCTEXT("ObjectMixerTabLabel", "Object Mixer");

	MenuItemName = LOCTEXT("ObjectMixerEditorMenuItem", "Object Mixer");
	MenuItemIcon =
		FSlateIcon(FObjectMixerEditorStyle::Get().GetStyleSetName(), "ObjectMixer.ToolbarButton", "ObjectMixer.ToolbarButton.Small");
	MenuItemTooltip = LOCTEXT("ObjectMixerEditorMenuItemTooltip", "Open an Object Mixer instance.");

	// Should be hidden for now since it's not ready yet for public release
	TabSpawnerType = ETabSpawnerMenuType::Hidden;
}

void FObjectMixerEditorModule::RegisterTabSpawner()
{
	FTabSpawnerEntry& BrowserSpawnerEntry =
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
			GetTabSpawnerId(), 
			FOnSpawnTab::CreateRaw(this, &FObjectMixerEditorModule::SpawnTab)
		)
		.SetIcon(MenuItemIcon)
		.SetDisplayName(MenuItemName)
		.SetTooltipText(MenuItemTooltip)
		.SetMenuType(TabSpawnerType)
	;

	// Always use the base ObjectMixer function call or WorkspaceGroup may be null 
	if (!FObjectMixerEditorModule::Get().RegisterItemInMenuGroup(BrowserSpawnerEntry))
	{
		BrowserSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory());
	}
}

FName FObjectMixerEditorModule::GetTabSpawnerId()
{
	return "ObjectMixerToolkit";
}

bool FObjectMixerEditorModule::RegisterItemInMenuGroup(FWorkspaceItem& InItem)
{
	if (WorkspaceGroup)
	{
		WorkspaceGroup->AddItem(MakeShareable(&InItem));
		
		return true;
	}

	return false;
}

void FObjectMixerEditorModule::UnregisterTabSpawner()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(GetTabSpawnerId());
}

void FObjectMixerEditorModule::RegisterSettings() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		// User Project Settings
		const TSharedPtr<ISettingsSection> ProjectSettingsSectionPtr = SettingsModule->RegisterSettings(
			"Editor", "Plugins", "Object Mixer",
			LOCTEXT("ObjectMixerSettingsDisplayName", "Object Mixer"),
			LOCTEXT("ObjectMixerSettingsDescription", "Configure Object Mixer user settings"),
			GetMutableDefault<UObjectMixerEditorSettings>());	
	}
}

void FObjectMixerEditorModule::UnregisterSettings() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Editor", "Plugins", "Object Mixer");
	}
}

TSharedRef<SDockTab> FObjectMixerEditorModule::SpawnTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> NewDockTab = SAssignNew(DockTab, SDockTab)
		.Label(TabLabel)
		.TabRole(ETabRole::NomadTab)
	;

	RegenerateListWidget();
			
	return NewDockTab;
}

TSharedPtr<FWorkspaceItem> FObjectMixerEditorModule::GetWorkspaceGroup()
{
	return WorkspaceGroup;
}

const TSubclassOf<UObjectMixerObjectFilter>& FObjectMixerEditorModule::GetDefaultFilterClass() const
{
	return DefaultFilterClass;
}

#undef LOCTEXT_NAMESPACE
