// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectMixerEditorSerializedData.h"

#include "Algo/Find.h"
#include "Algo/IndexOf.h"
#include "Algo/RemoveIf.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectMixerEditorSerializedData)

#define LOCTEXT_NAMESPACE "ObjectMixerEditor"

FName UObjectMixerEditorSerializedData::AllCollectionName = TEXT("All");

FObjectMixerSerializationDataPerFilter* UObjectMixerEditorSerializedData::FindSerializationDataByFilterClassName(const FName& FilterClassName)
{
	if (FObjectMixerSerializationDataPerFilter* Match = Algo::FindByPredicate(SerializedDataPerFilter,
		[FilterClassName](const FObjectMixerSerializationDataPerFilter& Comparator)
		{
			return Comparator.FilterClassName.IsEqual(FilterClassName);
		}))
	{
		return Match;
	}

	return nullptr;
}

bool UObjectMixerEditorSerializedData::AddObjectsToCollection(const FName& FilterClassName, const FName& CollectionName, const TSet<FSoftObjectPath>& ObjectsToAdd)
{
	if (CollectionName == NAME_None ||
		CollectionName.IsEqual(UObjectMixerEditorSerializedData::AllCollectionName, ENameCase::IgnoreCase) ||
		CollectionName.ToString().TrimStartAndEnd().IsEmpty())
	{
		return false;
	}
	
	FScopedTransaction AddObjectsToCollectionTransaction(LOCTEXT("AddObjectsToCollectionTransaction","Add Objects To Collection"));

	Modify();

	FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName);

	if (!Data)
	{
		SerializedDataPerFilter.Add({FilterClassName});
		Data = FindSerializationDataByFilterClassName(FilterClassName);
	}

	if (Data)
	{
		TArray<FObjectMixerCollectionObjectData> NewObjectsData;
		for (const FSoftObjectPath& Object : ObjectsToAdd)
		{
			NewObjectsData.Add({Object});
		}
	
		if (FObjectMixerCollectionObjectSet* Match = Algo::FindByPredicate(Data->SerializedCollections,
			[CollectionName](const FObjectMixerCollectionObjectSet& Comparator)
			{
				return Comparator.CollectionName.IsEqual(CollectionName);
			}))
		{
			NewObjectsData.SetNum(Algo::StableRemoveIf(NewObjectsData,
				[Match](const FObjectMixerCollectionObjectData& Other)
				{
					return Match->CollectionObjects.Contains(Other);
				}));
		
			Match->CollectionObjects.Append(NewObjectsData);
		}
		else
		{
			Data->SerializedCollections.Add({CollectionName, NewObjectsData});
		}

		SaveConfig();

		OnObjectMixerCollectionMapChanged.Broadcast();

		return true;
	}

	return false;
}

bool UObjectMixerEditorSerializedData::RemoveObjectsFromCollection(const FName& FilterClassName, const FName& CollectionName, const TSet<FSoftObjectPath>& ObjectsToRemove)
{
	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{		
		if (FObjectMixerCollectionObjectSet* Match = Algo::FindByPredicate(Data->SerializedCollections,
			[CollectionName](const FObjectMixerCollectionObjectSet& Comparator)
			{
				return Comparator.CollectionName.IsEqual(CollectionName);
			}))
		{
			FScopedTransaction RemoveObjectsFromCollectionTransaction(
				LOCTEXT("RemoveObjectsFromCollectionTransaction_Format","Remove {0}|plural(one=Object,other=Objects) From Collection"));
			
			Modify();

			Match->CollectionObjects.SetNum(Algo::StableRemoveIf(Match->CollectionObjects,
			[&ObjectsToRemove](const FObjectMixerCollectionObjectData& Comparator)
			{
				return ObjectsToRemove.Contains(Comparator.ObjectPath);
			}));
			
			SaveConfig();

			OnObjectMixerCollectionMapChanged.Broadcast();

			return true;
		}
		
	}

	return false;
}

bool UObjectMixerEditorSerializedData::RemoveCollection(const FName& FilterClassName, const FName& CollectionName)
{	
	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{
		FScopedTransaction RemoveCollectionTransaction(LOCTEXT("RemoveCollectionTransaction","Remove Collection"));
	
		Modify();
		
		Data->SerializedCollections.SetNum(Algo::StableRemoveIf(Data->SerializedCollections,
			[CollectionName](const FObjectMixerCollectionObjectSet& Comparator)
			{
				return Comparator.CollectionName.IsEqual(CollectionName);
			}));
		
		SaveConfig();

		OnObjectMixerCollectionMapChanged.Broadcast();

		return true;
	}

	return false;
}

bool UObjectMixerEditorSerializedData::DuplicateCollection(const FName& FilterClassName,
	const FName& CollectionToDuplicateName, FName& DesiredDuplicateName)
{
	uint32 NumberToAppendToCollectionName = 0;

	FName DuplicateName = FName(DesiredDuplicateName.ToString() + FString::FromInt(NumberToAppendToCollectionName));
	
	while (DoesCollectionExist(FilterClassName, DuplicateName))
	{
		NumberToAppendToCollectionName++;

		DuplicateName = FName(DesiredDuplicateName.ToString() + FString::FromInt(NumberToAppendToCollectionName));
	}

	DesiredDuplicateName = DuplicateName;

	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{
		FObjectMixerCollectionObjectSet MatchCopy;
		if (const FObjectMixerCollectionObjectSet* Match = Algo::FindByPredicate(Data->SerializedCollections,
			[CollectionToDuplicateName](const FObjectMixerCollectionObjectSet& Comparator)
			{
				return Comparator.CollectionName.IsEqual(CollectionToDuplicateName);
			}))
		{
			
			FScopedTransaction DuplicateCollectionTransaction(LOCTEXT("DuplicateCollectionTransaction","Duplicate Collection"));
	
			Modify();
			
			MatchCopy = (*Match);

			MatchCopy.CollectionName = DuplicateName;

			Data->SerializedCollections.Add(MatchCopy);
		
			SaveConfig();

			OnObjectMixerCollectionMapChanged.Broadcast();

			return true;
		}
	}

	return false;
}

bool UObjectMixerEditorSerializedData::ReorderCollection(const FName& FilterClassName,
                                                         const FName& CollectionToMoveName, const FName& CollectionInsertBeforeName)
{
	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{
		FObjectMixerCollectionObjectSet MatchCopy;
		bool bFoundMatch = false;
		if (const FObjectMixerCollectionObjectSet* Match = Algo::FindByPredicate(Data->SerializedCollections,
			[CollectionToMoveName](const FObjectMixerCollectionObjectSet& Comparator)
			{
				return Comparator.CollectionName.IsEqual(CollectionToMoveName);
			}))
		{
			MatchCopy = (*Match);
			bFoundMatch = true;
		}

		if (bFoundMatch)
		{
			FScopedTransaction ReorderCollectionTransaction(LOCTEXT("ReorderCollectionTransaction","Reorder Collection"));
	
			Modify();
			
			RemoveCollection(FilterClassName, CollectionToMoveName);

			if (CollectionInsertBeforeName == UObjectMixerEditorSerializedData::AllCollectionName) // Move MatchCopy to the end
			{
				Data->SerializedCollections.Add(MatchCopy);
			}
			else
			{
				const int32 IndexOfCollectionInsertBefore = Algo::IndexOfByPredicate(Data->SerializedCollections,
					[CollectionInsertBeforeName](const FObjectMixerCollectionObjectSet& Comparator)
					{
						return Comparator.CollectionName.IsEqual(CollectionInsertBeforeName);
					});

				Data->SerializedCollections.Insert(MatchCopy, IndexOfCollectionInsertBefore);
			}
		
			SaveConfig();

			OnObjectMixerCollectionMapChanged.Broadcast();

			return true;
		}
	}

	return false;
}

bool UObjectMixerEditorSerializedData::RenameCollection(const FName& FilterClassName,
	const FName CollectionNameToRename, const FName NewCollectionName)
{
	if (NewCollectionName == NAME_None ||
		NewCollectionName.IsEqual(UObjectMixerEditorSerializedData::AllCollectionName, ENameCase::IgnoreCase) ||
		NewCollectionName.ToString().TrimStartAndEnd().IsEmpty() ||
		DoesCollectionExist(FilterClassName, NewCollectionName))
	{
		return false;
	}
		
	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{
		if (FObjectMixerCollectionObjectSet* Match = Algo::FindByPredicate(Data->SerializedCollections,
			[CollectionNameToRename](const FObjectMixerCollectionObjectSet& Comparator)
			{
				return Comparator.CollectionName.IsEqual(CollectionNameToRename);
			}))
		{
			FScopedTransaction RenameCollectionTransaction(LOCTEXT("RenameCollectionTransaction","Rename Collection"));

			Modify();

			Match->CollectionName = NewCollectionName;
	
			SaveConfig();

			OnObjectMixerCollectionRenamed.Broadcast(CollectionNameToRename, NewCollectionName);

			OnObjectMixerCollectionMapChanged.Broadcast();

			return true;
		}
	}

	return false;
}

bool UObjectMixerEditorSerializedData::DoesCollectionExist(const FName& FilterClassName, const FName& CollectionName)
{
	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{
		return Algo::FindByPredicate(Data->SerializedCollections,
			[CollectionName](const FObjectMixerCollectionObjectSet& Comparator)
			{
				return Comparator.CollectionName.IsEqual(CollectionName);
			}) != nullptr;
	}

	return false;
}

bool UObjectMixerEditorSerializedData::IsObjectInCollection(const FName& FilterClassName, const FName& CollectionName, const FSoftObjectPath& InObject)
{
	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{
		if (const FObjectMixerCollectionObjectSet* Match = Algo::FindByPredicate(Data->SerializedCollections,
			[CollectionName](const FObjectMixerCollectionObjectSet& Comparator)
			{
				return Comparator.CollectionName.IsEqual(CollectionName);
			}))
		{
			return Algo::FindByPredicate(Match->CollectionObjects,
				[InObject](const FObjectMixerCollectionObjectData& Comparator)
				{
					return Comparator.ObjectPath == InObject;
				}) != nullptr;
		}
	}

	return false;
}

TSet<FName> UObjectMixerEditorSerializedData::GetCollectionsForObject(const FName& FilterClassName, const FSoftObjectPath& InObject)
{
	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{
		TSet<FName> CollectionsWithObject;

		for (const FObjectMixerCollectionObjectSet& CollectionObjectSet : Data->SerializedCollections)
		{
			if (IsObjectInCollection(FilterClassName, CollectionObjectSet.CollectionName, InObject))
			{
				CollectionsWithObject.Add(CollectionObjectSet.CollectionName);
			}
		}

		return CollectionsWithObject;
	}

	return {};
}

TArray<FName> UObjectMixerEditorSerializedData::GetAllCollectionNames(const FName& FilterClassName)
{
	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{
		TArray<FName> Collections;

		for (const FObjectMixerCollectionObjectSet& CollectionObjectSet : Data->SerializedCollections)
		{
			Collections.Add(CollectionObjectSet.CollectionName);
		}

		return Collections;
	}

	return {};
}

void UObjectMixerEditorSerializedData::SetShouldShowColumn(const FName& FilterClassName, const FName& ColumnName,
	const bool bNewShouldShowColumn)
{
	FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName);

	if (!Data)
	{
		SerializedDataPerFilter.Add({FilterClassName});
		Data = FindSerializationDataByFilterClassName(FilterClassName);
	}
	
	if (Data)
	{
		if (FObjectMixerColumnData* Match = Algo::FindByPredicate(Data->SerializedColumnData,
			[ColumnName](const FObjectMixerColumnData& Comparator)
			{
				return Comparator.ColumnName.IsEqual(ColumnName);
			}))
		{
			(*Match).bShouldBeEnabled = bNewShouldShowColumn;
		}
		else
		{
			Data->SerializedColumnData.Add({ColumnName, bNewShouldShowColumn});
		}

		SaveConfig();
	}
}

bool UObjectMixerEditorSerializedData::IsColumnDataSerialized(const FName& FilterClassName, const FName& ColumnName)
{
	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{
		return Algo::FindByPredicate(Data->SerializedColumnData,
			[ColumnName](const FObjectMixerColumnData& Comparator)
			{
				return Comparator.ColumnName.IsEqual(ColumnName);
			}) != nullptr;
	}

	return false;
}

bool UObjectMixerEditorSerializedData::ShouldShowColumn(const FName& FilterClassName, const FName& ColumnName)
{
	if (FObjectMixerSerializationDataPerFilter* Data = FindSerializationDataByFilterClassName(FilterClassName))
	{
		if (const FObjectMixerColumnData* Match = Algo::FindByPredicate(Data->SerializedColumnData,
			[ColumnName](const FObjectMixerColumnData& Comparator)
			{
				return Comparator.ColumnName.IsEqual(ColumnName);
			}))
		{
			return (*Match).bShouldBeEnabled;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
