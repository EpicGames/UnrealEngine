// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectMixerFilterFactory.h"

#include "AssetTypeCategories.h"
#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"

#include "Engine/Blueprint.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Engine/BlueprintGeneratedClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectMixerFilterFactory)

#define LOCTEXT_NAMESPACE "ObjectMixerEditor"

UObjectMixerBlueprintFilterFactory::UObjectMixerBlueprintFilterFactory()
{
	SupportedClass = UBlueprint::StaticClass();
	ParentClass = UObjectMixerBlueprintObjectFilter::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
}

FText UObjectMixerBlueprintFilterFactory::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Object Mixer Filter");
}

UObject* UObjectMixerBlueprintFilterFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UBlueprint* FilterBlueprint = nullptr;
	if (ensure(SupportedClass == Class))
	{
		ensure(0 != (RF_Public & Flags));
		FilterBlueprint = FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), NAME_None);
	}
	return FilterBlueprint;	
}

FString UObjectMixerBlueprintFilterFactory::GetDefaultNewAssetName() const
{
	return "NewObjectMixerFilter";
}

uint32 UObjectMixerBlueprintFilterFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Misc;
}

FText UObjectMixerBlueprintFilterFactory::GetToolTip() const
{
	return LOCTEXT("ObjectMixerFilterFactoryTooltip", "Creates a new Object Mixer Filter that can be used to define rules that govern what objects, columns and properties appear in Object Mixer.");
}

FString UObjectMixerBlueprintFilterFactory::GetToolTipDocumentationExcerpt() const
{
	return TEXT("UObjectMixerBlueprintFilter");
}

TArray<FAssetCategoryPath> UObjectMixerBlueprintFilterFactory::GetAssetMenuPathsForCategory(FName InCategory) const
{
	if (ParentClass)
	{
		return GetAssetMenuPaths(ParentClass, InCategory);
	}
	return Super::GetAssetMenuPathsForCategory(InCategory);
}

#undef LOCTEXT_NAMESPACE
