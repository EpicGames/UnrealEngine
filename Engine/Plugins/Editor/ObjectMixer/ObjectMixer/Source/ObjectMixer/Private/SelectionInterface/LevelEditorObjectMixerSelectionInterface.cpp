// Copyright Epic Games, Inc. All Rights Reserved.

#include "SelectionInterface/LevelEditorObjectMixerSelectionInterface.h"

#include "Algo/Transform.h"
#include "Editor.h"
#include "Engine/Selection.h"

FLevelEditorObjectMixerSelectionInterface::FLevelEditorObjectMixerSelectionInterface()
{
	USelection::SelectionChangedEvent.AddRaw(this, &FLevelEditorObjectMixerSelectionInterface::OnLevelSelectionChanged);
	USelection::SelectObjectEvent.AddRaw(this, &FLevelEditorObjectMixerSelectionInterface::OnLevelSelectionChanged);
}

FLevelEditorObjectMixerSelectionInterface::~FLevelEditorObjectMixerSelectionInterface()
{
	USelection::SelectionChangedEvent.RemoveAll(this);
	USelection::SelectObjectEvent.RemoveAll(this);
}

void FLevelEditorObjectMixerSelectionInterface::SelectActors(const TArray<AActor*>& InSelectedActors, bool bShouldSelect, bool bSelectEvenIfHidden)
{
	GEditor->GetSelectedActors()->Modify();

	// We'll batch selection changes instead by using BeginBatchSelectOperation()
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();

	if (bShouldSelect)
	{
		// Clear the selection
		GEditor->GetSelectedActors()->DeselectAll();
	}

	for (AActor* Actor : InSelectedActors)
	{
		constexpr bool bNotifyAfterSelect = false;
		GEditor->SelectActor(Actor, bShouldSelect, bNotifyAfterSelect, bSelectEvenIfHidden);
	}

	// Commit selection changes
	GEditor->GetSelectedActors()->EndBatchSelectOperation(/*bNotify*/false);

	GEditor->NoteSelectionChange();
}

void FLevelEditorObjectMixerSelectionInterface::SelectComponents(const TArray<UActorComponent*>& InSelectedComponents, bool bShouldSelect, bool bSelectEvenIfHidden)
{
	GEditor->GetSelectedComponents()->Modify();

	// We'll batch selection changes instead by using BeginBatchSelectOperation()
	GEditor->GetSelectedComponents()->BeginBatchSelectOperation();

	if (bShouldSelect)
	{
		// Clear the selection
		GEditor->GetSelectedComponents()->DeselectAll();
	}

	for (UActorComponent* Component : InSelectedComponents)
	{
		constexpr bool bNotifyAfterSelect = false;
		GEditor->SelectComponent(Component, bShouldSelect, bNotifyAfterSelect, bSelectEvenIfHidden);
	}

	// Commit selection changes
	GEditor->GetSelectedComponents()->EndBatchSelectOperation(/*bNotify*/false);

	GEditor->NoteSelectionChange();
}

TArray<AActor*> FLevelEditorObjectMixerSelectionInterface::GetSelectedActors() const
{
	TArray<AActor*> OutActors;
	GEditor->GetSelectedActors()->GetSelectedObjects<AActor>(OutActors);
	return MoveTemp(OutActors);
}

TArray<UActorComponent*> FLevelEditorObjectMixerSelectionInterface::GetSelectedComponents() const
{
	TArray<UActorComponent*> OutComponents;
	GEditor->GetSelectedComponents()->GetSelectedObjects<UActorComponent>(OutComponents);
	return MoveTemp(OutComponents);
}

void FLevelEditorObjectMixerSelectionInterface::OnLevelSelectionChanged(UObject* Obj)
{
	SelectionChanged.Broadcast();
}
