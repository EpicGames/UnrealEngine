// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/Modes/ObjectMixerOutlinerHierarchy.h"

#include "Views/List/ObjectMixerEditorList.h"
#include "Views/List/Modes/ObjectMixerOutlinerMode.h"
#include "Views/List/ObjectMixerUtils.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowActor.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowComponent.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowFolder.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowUObject.h"

#include "ActorDescTreeItem.h"
#include "ActorFolder.h"
#include "ActorFolderTreeItem.h"
#include "ActorMode.h"
#include "ActorTreeItem.h"
#include "ComponentTreeItem.h"
#include "EditorActorFolders.h"
#include "ISceneOutlinerMode.h"
#include "LevelTreeItem.h"
#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"
#include "ObjectMixerEditorSettings.h"
#include "WorldTreeItem.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Modules/ModuleManager.h"
#include "SceneOutlinerFwd.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowContainer.h"
#include "WorldPartition/ActorDescContainerInstanceCollection.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionSubsystem.h"

TUniquePtr<FObjectMixerOutlinerHierarchy> FObjectMixerOutlinerHierarchy::Create(ISceneOutlinerMode* Mode, const TWeakObjectPtr<UWorld>& World)
{
	FObjectMixerOutlinerHierarchy* Hierarchy = new FObjectMixerOutlinerHierarchy(Mode, World);

	GEngine->OnLevelActorAdded().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnLevelActorAdded);
	GEngine->OnLevelActorDeleted().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnLevelActorDeleted);
	GEngine->OnLevelActorDetached().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnLevelActorDetached);
	GEngine->OnLevelActorAttached().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnLevelActorAttached);
	GEngine->OnLevelActorFolderChanged().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnLevelActorFolderChanged);
	GEngine->OnLevelActorListChanged().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnLevelActorListChanged);
	GEngine->OnActorFolderAdded().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnActorFolderAdded);
	GEngine->OnActorFoldersUpdatedEvent().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnActorFoldersUpdatedEvent);

	IWorldPartitionEditorModule::Get().OnWorldPartitionCreated().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnWorldPartitionCreated);

	if (World.IsValid())
	{
		if (World->PersistentLevel)
		{
			World->PersistentLevel->OnLoadedActorAddedToLevelEvent.AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnLoadedActorAdded);
			World->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnLoadedActorRemoved);
		}

		World->OnWorldPartitionInitialized().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnWorldPartitionInitialized);
		World->OnWorldPartitionUninitialized().AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnWorldPartitionUninitialized);

		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			WorldPartition->OnActorDescInstanceRemovedEvent.AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnActorDescInstanceRemoved);
		}
	}

	FWorldDelegates::LevelAddedToWorld.AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnLevelAdded);
	FWorldDelegates::LevelRemovedFromWorld.AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnLevelRemoved);

	FActorFolders& Folders = FActorFolders::Get();
	Folders.OnFolderCreated.AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnBroadcastFolderCreate);
	Folders.OnFolderMoved.AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnBroadcastFolderMove);
	Folders.OnFolderDeleted.AddRaw(Hierarchy, &FObjectMixerOutlinerHierarchy::OnBroadcastFolderDelete);

	return TUniquePtr<FObjectMixerOutlinerHierarchy>(Hierarchy);
}

FObjectMixerOutlinerMode* FObjectMixerOutlinerHierarchy::GetCastedMode() const
{
	return StaticCast<FObjectMixerOutlinerMode*>(Mode);
}

const UObjectMixerObjectFilter* FObjectMixerOutlinerHierarchy::GetObjectFilter() const
{
	if (const TSharedPtr<FObjectMixerEditorList> ListModel = GetCastedMode()->GetListModelPtr().Pin())
	{
		return ListModel->GetMainObjectFilterInstance();
	}

	return nullptr;
}

FObjectMixerOutlinerHierarchy::FObjectMixerOutlinerHierarchy(ISceneOutlinerMode* Mode, const TWeakObjectPtr<UWorld>& World)
	: ISceneOutlinerHierarchy(Mode)
	, RepresentingWorld(World)
{
}

FObjectMixerOutlinerHierarchy::~FObjectMixerOutlinerHierarchy()
{
	if (GEngine)
	{
		GEngine->OnLevelActorAdded().RemoveAll(this);
		GEngine->OnLevelActorDeleted().RemoveAll(this);
		GEngine->OnLevelActorDetached().RemoveAll(this);
		GEngine->OnLevelActorAttached().RemoveAll(this);
		GEngine->OnLevelActorFolderChanged().RemoveAll(this);
		GEngine->OnLevelActorListChanged().RemoveAll(this);
		GEngine->OnActorFolderAdded().RemoveAll(this);
		GEngine->OnActorFoldersUpdatedEvent().RemoveAll(this);
	}

	if (RepresentingWorld.IsValid())
	{
		if (RepresentingWorld->PersistentLevel)
		{
			RepresentingWorld->PersistentLevel->OnLoadedActorAddedToLevelEvent.RemoveAll(this);
			RepresentingWorld->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.RemoveAll(this);
		}

		RepresentingWorld->OnWorldPartitionInitialized().RemoveAll(this);
		RepresentingWorld->OnWorldPartitionUninitialized().RemoveAll(this);

		if (UWorldPartition* WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			WorldPartition->OnActorDescInstanceRemovedEvent.RemoveAll(this);
		}
	}

	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);


	auto& Folders = FActorFolders::Get();
	Folders.OnFolderCreated.RemoveAll(this);
	Folders.OnFolderMoved.RemoveAll(this);
	Folders.OnFolderDeleted.RemoveAll(this);
}

FSceneOutlinerTreeItemPtr FObjectMixerOutlinerHierarchy::FindOrCreateParentItem(
	const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate)
{
	if (Item.IsA<FWorldTreeItem>())
	{
		return nullptr;
	}
	else if (const FActorTreeItem* ActorTreeItem = Item.CastTo<FActorTreeItem>())
	{
		// Override parent
		if (const FObjectMixerEditorListRowActor* ObjectMixerActorItem = Item.CastTo<FObjectMixerEditorListRowActor>())
		{
			if (AActor* ParentActor = ObjectMixerActorItem->OverrideParent.Get())
			{
				// If OverrideParent is set, assume this item was created in the process of creating the parent, so we
				// don't have to create the parent as a fallback here
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentActor))
				{
					return *ParentItem;
				}
			}
		}

		if (AActor* Actor = ActorTreeItem->Actor.Get())
		{
			// Parent Actor (Actor attachement / parenting)
			if (AActor* ParentActor = Actor->GetSceneOutlinerParent())
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentActor))
				{
					return *ParentItem;
				}
				// If Parent can be listed in SceneOutliner return nullptr so it gets created
				else if (ParentActor->IsListedInSceneOutliner())
				{
					if (bCreate)
					{
						if (FSceneOutlinerTreeItemPtr ActorItem =
							Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
								FObjectMixerEditorListRowActor(ParentActor, GetCastedMode()->GetSceneOutliner()), true))
						{
							return ActorItem;
						}
					}
					return nullptr;
				}
			}

			// Parent Folder
			FFolder Folder = Actor->GetFolder();
			if (Mode->ShouldShowFolders() && !Folder.IsNone())
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(Folder))
				{
					return *ParentItem;
				}
				else
				{
					return bCreate ? Mode->CreateItemFor<FObjectMixerEditorListRowFolder>(FObjectMixerEditorListRowFolder(Folder, GetCastedMode()->GetSceneOutliner(), ActorTreeItem->Actor->GetWorld()), true) : nullptr;
				}
			}

			// Parent Level Instance
			if (ILevelInstanceInterface* OwningLevelInstance = Cast<ILevelInstanceInterface>(Folder.GetRootObjectPtr()))
			{
				const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor);
				const bool bIsAnEditingLevelInstance = LevelInstance ? LevelInstance->IsEditing() : false;
				// Parent this to a LevelInstance if the parent LevelInstance is being edited or if this is a sub LevelInstance which is being edited
				if (bShowingLevelInstances || (OwningLevelInstance->IsEditing() || bIsAnEditingLevelInstance))
				{
					AActor* OwningActor = CastChecked<AActor>(OwningLevelInstance);
					if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(OwningActor))
					{
						return *ParentItem;
					}
					else
					{
						if (bCreate)
						{
							if (FSceneOutlinerTreeItemPtr ActorItem =
								Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
									FObjectMixerEditorListRowActor(OwningActor, GetCastedMode()->GetSceneOutliner()), true))
							{
								return ActorItem;
							}
						}
						return nullptr;
					}
				}
			}

			// Parent Level Using Actor Folders
			ULevel* OwningLevel = Cast<ULevel>(Folder.GetRootObjectPtr());
			// For the persistent level, fallback on the world
			if (FActorMode::IsActorLevelDisplayable(OwningLevel))
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(OwningLevel))
				{
					return *ParentItem;
				}
				else
				{
					return bCreate ? Mode->CreateItemFor<FLevelTreeItem>(OwningLevel, true) : nullptr;
				}
			}
		}
	}
	else if (const FActorFolderTreeItem* FolderItem = Item.CastTo<FActorFolderTreeItem>())
	{
		// We should never call FindParents on a folder item if folders are not being shown
		check(Mode->ShouldShowFolders());

		const FFolder ParentPath = FolderItem->GetFolder().GetParent();

		// Parent Folder
		if (!ParentPath.IsNone())
		{
			if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentPath))
			{
				return *ParentItem;
			}
			else
			{
				return bCreate ? Mode->CreateItemFor<FObjectMixerEditorListRowFolder>(FObjectMixerEditorListRowFolder(ParentPath, GetCastedMode()->GetSceneOutliner(), FolderItem->World.Get()), true) : nullptr;
			}
		}
		// Parent Level Instance
		else if (ILevelInstanceInterface* OwningLevelInstance = Cast<ILevelInstanceInterface>(ParentPath.GetRootObjectPtr()))
		{
			if (bShowingLevelInstances || OwningLevelInstance->IsEditing())
			{
				AActor* OwningActor = CastChecked<AActor>(OwningLevelInstance);
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(OwningActor))
				{
					return *ParentItem;
				}
				else
				{
					if (bCreate)
					{
						if (FSceneOutlinerTreeItemPtr ActorItem =
							Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
								FObjectMixerEditorListRowActor(OwningActor, GetCastedMode()->GetSceneOutliner()), true))
						{
							return ActorItem;
						}
					}
					return nullptr;
				}
			}
		}
		// Parent Level Using Actor Folders
		else if (ULevel* OwningLevel = Cast<ULevel>(ParentPath.GetRootObjectPtr()))
		{
			if (FActorMode::IsActorLevelDisplayable(OwningLevel))
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(OwningLevel))
				{
					return *ParentItem;
				}
				else
				{
					return bCreate ? Mode->CreateItemFor<FLevelTreeItem>(OwningLevel, true) : nullptr;
				}
			}
		}
	}
	else if (const FComponentTreeItem* ComponentTreeItem = Item.CastTo<FComponentTreeItem>())
	{
		if (AActor* Owner = ComponentTreeItem->Component->GetOwner())
		{
			if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(Owner))
			{
				return *ParentItem;
			}
			else
			{
				if (bCreate)
				{
					if (FSceneOutlinerTreeItemPtr ActorItem =
						Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
							FObjectMixerEditorListRowActor(Owner, GetCastedMode()->GetSceneOutliner()), true))
					{
						return ActorItem;
					}
				}
				return nullptr;
			}
		}
		// do not default to world on Component items
		return nullptr;
	}
	else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
	{
		if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
		{
			if (UWorld* RepresentingWorldPtr = RepresentingWorld.Get())
			{
				const FFolder ActorDescFolder = FActorFolders::GetActorDescInstanceFolder(*RepresentingWorldPtr, ActorDescInstance);
				if (!ActorDescFolder.IsNone())
				{
					if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ActorDescFolder))
					{
						return *ParentItem;
					}
					else
					{
						return bCreate ? Mode->CreateItemFor<FObjectMixerEditorListRowFolder>(FObjectMixerEditorListRowFolder(ActorDescFolder, GetCastedMode()->GetSceneOutliner(), RepresentingWorldPtr), true) : nullptr;
					}
				}
			}

			// Parent Actor (Actor attachment / parenting)
			const FGuid& ParentActorGuid = ActorDescInstance->GetSceneOutlinerParent();
			if (ParentActorGuid.IsValid())
			{
				if (UActorDescContainerInstance* ActorDescContainerInstance = ActorDescInstance->GetContainerInstance())
				{
					if (const FWorldPartitionActorDescInstance* ParentActorDescInstance = ActorDescContainerInstance->GetActorDescInstance(ParentActorGuid))
					{
						// If parent actor is loaded
						if (AActor* ParentActor = ParentActorDescInstance->GetActor())
						{
							// Find loaded parent actor node (from the object ptr)
							if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentActor))
							{
								return *ParentItem;
							}
							else
							{
								if (bCreate)
								{
									if (FSceneOutlinerTreeItemPtr ActorItem =
										Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
											FObjectMixerEditorListRowActor(ParentActor, GetCastedMode()->GetSceneOutliner()), true))
									{
										return ActorItem;
									}
								}
								return nullptr;
							}
						}

						// Find unloaded parent actor node (from the guid)
						if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(FActorDescTreeItem::ComputeTreeItemID(ParentActorGuid, ActorDescContainerInstance)))
						{
							return *ParentItem;
						}
						else
						{
							return bCreate ? Mode->CreateItemFor<FActorDescTreeItem>(FActorDescTreeItem(ParentActorGuid, ActorDescContainerInstance)) : nullptr;
						}
					}
				}
			}
			else if (UActorDescContainerInstance* ActorDescContainerInstance = ActorDescInstance->GetContainerInstance())
			{
				const ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(RepresentingWorld.Get());
				UWorld* OuterWorld = ActorDescContainerInstance->GetTypedOuter<UWorld>();
				// If parent actor is loaded
				if (AActor* ParentActor = OuterWorld ? Cast<AActor>(LevelInstanceSubsystem->GetOwningLevelInstance(OuterWorld->PersistentLevel)) : nullptr)
				{
					if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentActor))
					{
						return *ParentItem;
					}
					else
					{
						if (bCreate)
						{
							if (FSceneOutlinerTreeItemPtr ActorItem =
								Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
									FObjectMixerEditorListRowActor(ParentActor, GetCastedMode()->GetSceneOutliner()), true))
							{
								return ActorItem;
							}
						}
						return nullptr;
					}
				}
			}
		}
	}
	else if (const FObjectMixerEditorListRowUObject* ObjectItem = Item.CastTo<FObjectMixerEditorListRowUObject>())
	{
		if (UObject* Object = ObjectItem->ObjectSoftPtr.Get())
		{
			UObject* ParentObject = ObjectItem->bIncludeObjectStack ? Object->GetOuter() : Object->GetTypedOuter<AActor>();
			if (ParentObject)
			{
				if (ObjectItem->ContainerProperty != NAME_None)
				{
					FSceneOutlinerTreeItemID ContainerRowId = HashCombineFast(GetTypeHash(ParentObject), GetTypeHash(ObjectItem->ContainerProperty));
					if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ContainerRowId))
					{
						return *ParentItem;
					}
					else if (bCreate)
					{
						if (FSceneOutlinerTreeItemPtr ContainerItem =
							Mode->CreateItemFor<FObjectMixerEditorListRowContainer>(
								FObjectMixerEditorListRowContainer(ParentObject, ObjectItem->ContainerProperty, GetCastedMode()->GetSceneOutliner()), true))
						{
							return ContainerItem;
						}
					}

					return nullptr;
				}
				
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentObject))
				{
					return *ParentItem;
				}
				else if (bCreate)
				{
					if (ParentObject->IsA<AActor>())
					{
						if (FSceneOutlinerTreeItemPtr ActorItem =
								Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
									FObjectMixerEditorListRowActor(Cast<AActor>(ParentObject), GetCastedMode()->GetSceneOutliner()), true))
						{
							return ActorItem;
						}
					}
					else
					{
						if (TSharedPtr<FObjectMixerEditorListRowUObject> ParentObjectItem =
							StaticCastSharedPtr<FObjectMixerEditorListRowUObject>(Mode->CreateItemFor<FObjectMixerEditorListRowUObject>(
								FObjectMixerEditorListRowUObject(ParentObject, GetCastedMode()->GetSceneOutliner()), true)))
						{
							ParentObjectItem->bIncludeObjectStack = ObjectItem->bIncludeObjectStack;
							return ParentObjectItem;
						}
					}
				}
					
				return nullptr;
			}
		}
	}
	else if (const FObjectMixerEditorListRowContainer* ContainerItem = Item.CastTo<FObjectMixerEditorListRowContainer>())
	{
		if (UObject* PropertyOwner = ContainerItem->GetPropertyOwner())
		{
			if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(PropertyOwner))
			{
				return *ParentItem;
			}
			else if (bCreate)
			{
				if (PropertyOwner->IsA<AActor>())
				{
					if (FSceneOutlinerTreeItemPtr ActorItem =
							Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
								FObjectMixerEditorListRowActor(Cast<AActor>(PropertyOwner), GetCastedMode()->GetSceneOutliner()), true))
					{
						return ActorItem;
					}
				}
				else
				{
					if (TSharedPtr<FObjectMixerEditorListRowUObject> ParentObjectItem =
						StaticCastSharedPtr<FObjectMixerEditorListRowUObject>(Mode->CreateItemFor<FObjectMixerEditorListRowUObject>(
							FObjectMixerEditorListRowUObject(PropertyOwner, GetCastedMode()->GetSceneOutliner()), true)))
					{
						// Assume that, since a container item exists, that it is part of a full sub-object stack
						ParentObjectItem->bIncludeObjectStack = true;
						return ParentObjectItem;
					}
				}
			}
					
			return nullptr;
		}
	}

	// If we get here return world item
	if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(RepresentingWorld.Get()))
	{
		return *ParentItem;
	}
	else
	{
		return bCreate ? Mode->CreateItemFor<FWorldTreeItem>(RepresentingWorld.Get(), true) : nullptr;
	}
}

bool FObjectMixerOutlinerHierarchy::DoesWorldObjectHaveAcceptableClass(const UObject* Object) const
{
	if (IsValid(Object))
	{
		for (UClass* Class : GetCastedMode()->GetListModelPtr().Pin()->ObjectClassesToFilterCache)
		{
			if (Object->IsA(Class))
			{
				return true;
			}
		}
	}

	return false;
}

void FObjectMixerOutlinerHierarchy::CreateFolderChild(const FFolder& Folder, UWorld* World, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
    if (FSceneOutlinerTreeItemPtr FolderItem =
        Mode->CreateItemFor<FObjectMixerEditorListRowFolder>(
            FObjectMixerEditorListRowFolder(Folder, GetCastedMode()->GetSceneOutliner(), World)))
    {
        OutItems.Add(FolderItem);
    }
}

void FObjectMixerOutlinerHierarchy::ForEachActorInLevel(
	AActor* Actor, const ULevelInstanceSubsystem* LevelInstanceSubsystem, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	if (!Actor)
	{
		return;
	}
	
	// If we are not showing LevelInstances, LevelInstance sub actor items should not be created unless they belong to a LevelInstance which is being edited
	if (LevelInstanceSubsystem)
	{
		if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
		{
			if (!bShowingLevelInstances && !ParentLevelInstance->IsEditing())
			{
				return;
			}
		}
	}

	OutItems.Append(ConditionallyCreateActorAndComponentItems(Actor));
}

void FObjectMixerOutlinerHierarchy::CreateWorldChildren(UWorld* World, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	check(World);

	if (Mode->ShouldShowFolders() && bShowingEmptyFolders)
	{
		// Add any folders which might match the current search terms
		FActorFolders::Get().ForEachFolder(*World, [this, World, &OutItems](const FFolder& Folder)
		{
			CreateFolderChild(Folder, World, OutItems);
			return true;
		});
	}
	
	const ULevelInstanceSubsystem* LevelInstanceSubsystem = World->GetSubsystem<ULevelInstanceSubsystem>();
	
	for (const ULevel* Level : World->GetLevels())
	{
		if (!Level || !Level->bIsVisible)
		{
			continue;
		}

		ForEachObjectWithOuter(Level,
		  [&](UObject* Object)
		{
			if (AActor* Actor = Cast<AActor>(Object))
			{
				ForEachActorInLevel(Actor, LevelInstanceSubsystem, OutItems);
			}
			// Since components are created per actor already, we only are concerned with other matching UObjects
			else if (!Object->IsA(UActorComponent::StaticClass()) && DoesWorldObjectHaveAcceptableClass(Object)) 
			{
				if (FSceneOutlinerTreeItemPtr ObjectItem =
					Mode->CreateItemFor<FObjectMixerEditorListRowUObject>(
						FObjectMixerEditorListRowUObject(Object, GetCastedMode()->GetSceneOutliner())))
				{
					OutItems.Add(ObjectItem);
				}
			}
		});
	}

	if (bShowingUnloadedActors)
	{
		if (UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(World))
		{
			auto ForEachWorldPartition = [this, LevelInstanceSubsystem, &OutItems](UWorldPartition* WorldPartition)
			{
				// Skip unloaded actors if they are part of a non editing level instance and the outliner hides the content of level instances
				if (!bShowingLevelInstances)
				{
					UWorld* OuterWorld = WorldPartition->GetTypedOuter<UWorld>();
					ULevel* OuterLevel = OuterWorld ? OuterWorld->PersistentLevel.Get() : nullptr;
					ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem ? LevelInstanceSubsystem->GetOwningLevelInstance(OuterLevel) : nullptr;
					if (LevelInstance && !LevelInstance->IsEditing())
					{
						return true;
					}
				}
				FWorldPartitionHelpers::ForEachActorDescInstance(WorldPartition,
					[this, &OutItems](const FWorldPartitionActorDescInstance* ActorDescInstance)
				{
					if (ActorDescInstance != nullptr && !ActorDescInstance->IsLoaded(true))
					{
						if (const FSceneOutlinerTreeItemPtr ActorDescItem =
							Mode->CreateItemFor<FActorDescTreeItem>(FActorDescTreeItem(ActorDescInstance->GetGuid(), ActorDescInstance->GetContainerInstance())))
						{
							OutItems.Add(ActorDescItem);
						}
					}
					return true;
				});
				return true;
			};
			
			WorldPartitionSubsystem->ForEachWorldPartition(ForEachWorldPartition);
		}
	}
}

bool FObjectMixerOutlinerHierarchy::CreateComponentItems(AActor* Actor, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	bool bAnyItemsBeforeFiltering = false;

	check(Actor);
	// Add all this actors components if showing components and the owning actor was created
	if (bShowingComponents)
	{
		TArray<UActorComponent*> ActorComponents;
		Actor->GetComponents(ActorComponents);
		for (UActorComponent* Component : ActorComponents)
		{
			if (IsValid(Component) && DoesWorldObjectHaveAcceptableClass(Component))
			{
				bAnyItemsBeforeFiltering = true;

				if (FSceneOutlinerTreeItemPtr ComponentItem =
					Mode->CreateItemFor<FObjectMixerEditorListRowComponent>(
						FObjectMixerEditorListRowComponent(Component, GetCastedMode()->GetSceneOutliner())))
				{
					OutItems.Add(ComponentItem);
				}
			}
		}
	}

	return bAnyItemsBeforeFiltering;
}

bool FObjectMixerOutlinerHierarchy::CreateAssociatedActorItems(AActor* Actor, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	bool bAnyItemsBeforeFiltering = false;

	check(Actor);
	if (const UObjectMixerObjectFilter* Filter = GetObjectFilter())
	{
		const TArray<AActor*> AssociatedActors = Filter->FindAssociatedActors(Actor);

		for (AActor* AssociatedActor : AssociatedActors)
		{
			if (IsValid(Actor))
			{
				bAnyItemsBeforeFiltering = true;

				if (FSceneOutlinerTreeItemPtr ObjectItem =
					Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
						FObjectMixerEditorListRowActor(AssociatedActor, GetCastedMode()->GetSceneOutliner(), FText::GetEmpty(), Actor)))
				{
					OutItems.Add(ObjectItem);
				}
			}
		}
	}

	return bAnyItemsBeforeFiltering;
}

bool FObjectMixerOutlinerHierarchy::CreateSubObjectItems(AActor* Actor, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	bool bAnyItemsBeforeFiltering = false;
	
	check(Actor);
	if (const UObjectMixerObjectFilter* Filter = GetObjectFilter())
	{
		const TArray<FObjectMixerEditorActorSubObject> SubObjects = Filter->GetActorSubObjects(Actor);

		for (const FObjectMixerEditorActorSubObject& SubObject : SubObjects)
		{
			if (TStrongObjectPtr<UObject> Object = SubObject.Object.Pin())
			{
				bAnyItemsBeforeFiltering = true;

				if (SubObject.bIncludeObjectStack)
				{
					bool bIsValidSubObjectStack = false;
					TArray<UObject*> SubObjectStack;
					UObject* Outer = Object.Get()->GetOuter();
					while (Outer != Actor && Outer != nullptr)
					{
						SubObjectStack.Add(Outer);
						Outer = Outer->GetOuter();
						if (Outer == Actor)
						{
							bIsValidSubObjectStack = true;
						}
					}

					if (bIsValidSubObjectStack)
					{
						for (int32 Index = 0; Index < SubObjectStack.Num(); ++Index)
						{
							if (FSceneOutlinerTreeItemPtr ObjectItem =
								Mode->CreateItemFor<FObjectMixerEditorListRowUObject>(
									FObjectMixerEditorListRowUObject(SubObjectStack[Index], GetCastedMode()->GetSceneOutliner(), FText::GetEmpty())))
							{
								OutItems.Add(ObjectItem);
							}
						}
					}
				}
				
				if (SubObject.ContainerProperty.PropertyName != NAME_None)
				{
					if (TSharedPtr<FObjectMixerEditorListRowContainer> ContainerItem =
						StaticCastSharedPtr<FObjectMixerEditorListRowContainer>(Mode->CreateItemFor<FObjectMixerEditorListRowContainer>(
							FObjectMixerEditorListRowContainer(Object->GetOuter(), SubObject.ContainerProperty.PropertyName, GetCastedMode()->GetSceneOutliner()))))
					{
						ContainerItem->PropertyIconOverride = SubObject.ContainerProperty.PropertyIconOverride;
						OutItems.Add(ContainerItem);
					}				
				}
				
				if (TSharedPtr<FObjectMixerEditorListRowUObject> ObjectItem =
					StaticCastSharedPtr<FObjectMixerEditorListRowUObject>(Mode->CreateItemFor<FObjectMixerEditorListRowUObject>(
						FObjectMixerEditorListRowUObject(Object.Get(), GetCastedMode()->GetSceneOutliner(), SubObject.DisplayName))))
				{
					ObjectItem->ContainerProperty = SubObject.ContainerProperty.PropertyName;
					ObjectItem->bIncludeObjectStack = SubObject.bIncludeObjectStack;
					OutItems.Add(ObjectItem);
				}
			}
		}
	}
	
	return bAnyItemsBeforeFiltering;
}

TArray<FSceneOutlinerTreeItemPtr> FObjectMixerOutlinerHierarchy::ConditionallyCreateActorAndComponentItems(AActor* Actor) const
{
	if (!IsValid(Actor))
	{
		return {};
	}
	
	// Whether or not we have children to return, we should create an actor row if children would be returned if there were no filters
	bool bWouldReturnAnyChildItemsBeforeFiltering = false;
	
	TArray<FSceneOutlinerTreeItemPtr> ComponentRows;
	bWouldReturnAnyChildItemsBeforeFiltering |= CreateComponentItems(Actor, ComponentRows);

	TArray<FSceneOutlinerTreeItemPtr> AssociatedActorRows;
	bWouldReturnAnyChildItemsBeforeFiltering |= CreateAssociatedActorItems(Actor, AssociatedActorRows);

	TArray<FSceneOutlinerTreeItemPtr> SubObjectRows;
	bWouldReturnAnyChildItemsBeforeFiltering |= CreateSubObjectItems(Actor, SubObjectRows);

	const bool bShouldCreateActorItem =
		bWouldReturnAnyChildItemsBeforeFiltering || !bShowingOnlyActorWithValidComponents || DoesWorldObjectHaveAcceptableClass(Actor);

	TArray<FSceneOutlinerTreeItemPtr> ReturnValue;
	if (bShouldCreateActorItem)
	{
		// Force the actor creation regardless of whether it passes the current filters, as one of its children did
		const bool bForce = true;
		if (const FSceneOutlinerTreeItemPtr ActorItem =
			Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
				FObjectMixerEditorListRowActor(Actor, GetCastedMode()->GetSceneOutliner()), bForce))
		{
			const bool bIsHybridModeEnabled = GetCastedMode()->ShouldAllowHybridRows() && GetDefault<UObjectMixerEditorSettings>()->IsHybridRowModeEnabled();
			if (bIsHybridModeEnabled && ComponentRows.Num() == 1 && AssociatedActorRows.IsEmpty()) // Create hybrid row
			{
				if (FObjectMixerEditorListRowActor* AsActorRow = FObjectMixerUtils::AsActorRow(ActorItem))
				{
					const FComponentTreeItem* ComponentItem = ComponentRows[0]->CastTo<FComponentTreeItem>();

					if (ComponentItem && ComponentItem->Component.IsValid())
					{
						AsActorRow->RowData.SetHybridComponent(ComponentItem->Component.Get());
					}
				}
			}
			else
			{
				ReturnValue.Append(ComponentRows);
				ReturnValue.Append(AssociatedActorRows);
				ReturnValue.Append(SubObjectRows);
			}

			// Place ActorItem before components
			ReturnValue.Insert(ActorItem, 0);
		}
	}

	return ReturnValue;
}

void FObjectMixerOutlinerHierarchy::CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	if (RepresentingWorld.IsValid())
	{
		UWorld* RepresentingWorldPtr = RepresentingWorld.Get();
		check(RepresentingWorldPtr);
		if (FSceneOutlinerTreeItemPtr WorldItem = Mode->CreateItemFor<FWorldTreeItem>(RepresentingWorldPtr))
		{
			OutItems.Add(WorldItem);
		}
		// Create world children regardless of if a world item was created
		CreateWorldChildren(RepresentingWorldPtr, OutItems);
	}
}

void FObjectMixerOutlinerHierarchy::CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const
{
	auto CreateChildrenFolders = [this](UWorld* InWorld, const FFolder& InParentFolder, const FFolder::FRootObject& InFolderRootObject, TArray<FSceneOutlinerTreeItemPtr>& OutChildren)
	{
		FActorFolders::Get().ForEachFolderWithRootObject(*InWorld, InFolderRootObject, [this, InWorld, &InParentFolder, &OutChildren](const FFolder& Folder)
		{
			if (Folder.IsChildOf(InParentFolder))
			{
				if (FSceneOutlinerTreeItemPtr NewFolderItem = Mode->CreateItemFor<FObjectMixerEditorListRowFolder>(FObjectMixerEditorListRowFolder(Folder, GetCastedMode()->GetSceneOutliner(), InWorld)))
				{
					OutChildren.Add(NewFolderItem);
				}
			}
			return true;
		});
	};
	
	UWorld* World = RepresentingWorld.Get();
	if (FWorldTreeItem* WorldItem = Item->CastTo<FWorldTreeItem>())
	{
		check(WorldItem->World == RepresentingWorld);
		CreateWorldChildren(WorldItem->World.Get(), OutChildren);
	}
	else if (const FActorTreeItem* ParentActorItem = Item->CastTo<FActorTreeItem>())
	{
		AActor* ParentActor = ParentActorItem->Actor.Get();
		if (ParentActor->GetWorld() != RepresentingWorld)
		{
			return;
		}

		CreateComponentItems(ParentActor, OutChildren);
		CreateAssociatedActorItems(ParentActor, OutChildren);
		CreateSubObjectItems(ParentActor, OutChildren);

		TArray<AActor*> ChildActors;

		if (const ILevelInstanceInterface* LevelInstanceParent = Cast<ILevelInstanceInterface>(ParentActor))
		{
			const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
			check(LevelInstanceSubsystem);

			LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstanceParent, [this, LevelInstanceParent, LevelInstanceSubsystem, &ChildActors](AActor* SubActor)
			{
				const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(SubActor);
				const bool bIsAnEditingLevelInstance = LevelInstance ? LevelInstanceSubsystem->IsEditingLevelInstance(LevelInstance) : false;
				if (bShowingLevelInstances || (LevelInstanceSubsystem->IsEditingLevelInstance(LevelInstanceParent) || bIsAnEditingLevelInstance))
				{
					ChildActors.Add(SubActor);
				}
				return true;
			});

			check(World == CastChecked<AActor>(LevelInstanceParent)->GetWorld());
			FFolder ParentFolder = ParentActor->GetFolder();
			CreateChildrenFolders(World, ParentFolder, ParentActor, OutChildren);
		}
		else
		{
			TFunction<bool(AActor*)> GetAttachedActors = [&ChildActors, &GetAttachedActors](AActor* Child)
			{
				ChildActors.Add(Child);
				Child->ForEachAttachedActors(GetAttachedActors);

				// Always continue
				return true;
			};

			// Grab all direct/indirect children of an actor
			ParentActor->ForEachAttachedActors(GetAttachedActors);
		}

		for (auto ChildActor : ChildActors)
		{
			if (FSceneOutlinerTreeItemPtr ChildActorItem =
				Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
					FObjectMixerEditorListRowActor(ChildActor, GetCastedMode()->GetSceneOutliner())))
			{
				OutChildren.Add(ChildActorItem);

				CreateComponentItems(ChildActor, OutChildren);
			}
		}
	}
	else if (FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>())
	{
		check(Mode->ShouldShowFolders());
		
		check(World == FolderItem->World.Get());
		FFolder ParentFolder = FolderItem->GetFolder();
		check(!ParentFolder.IsNone());
		CreateChildrenFolders(World, ParentFolder, ParentFolder.GetRootObject(), OutChildren);
	}
}

void FObjectMixerOutlinerHierarchy::FullRefreshEvent()
{
	FSceneOutlinerHierarchyChangedData EventData;
	EventData.Type = FSceneOutlinerHierarchyChangedData::FullRefresh;

	HierarchyChangedEvent.Broadcast(EventData);
}

void FObjectMixerOutlinerHierarchy::OnWorldPartitionCreated(UWorld* InWorld)
{
	if (RepresentingWorld.Get() == InWorld)
	{
		FullRefreshEvent();
	}
}

void FObjectMixerOutlinerHierarchy::OnLevelActorAdded(AActor* InActor)
{
	if (InActor != nullptr && RepresentingWorld.Get() == InActor->GetWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Added;
		if (FSceneOutlinerTreeItemPtr ActorItem =
			Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
				FObjectMixerEditorListRowActor(InActor, GetCastedMode()->GetSceneOutliner())))
		{
			EventData.Items.Add(ActorItem);
		}
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FObjectMixerOutlinerHierarchy::OnLevelActorDeleted(AActor* InActor)
{
	if (InActor != nullptr && RepresentingWorld.Get() == InActor->GetWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
		EventData.ItemIDs.Add(InActor);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FObjectMixerOutlinerHierarchy::OnLevelActorAttached(AActor* InActor, const AActor* InParent)
{
	if (InActor != nullptr && RepresentingWorld.Get() == InActor->GetWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Moved;
		EventData.ItemIDs.Add(InActor);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FObjectMixerOutlinerHierarchy::OnLevelActorDetached(AActor* InActor, const AActor* InParent)
{
	if (InActor != nullptr && RepresentingWorld.Get() == InActor->GetWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Moved;
		EventData.ItemIDs.Add(InActor);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FObjectMixerOutlinerHierarchy::OnLoadedActorAdded(AActor& InActor)
{
	OnLevelActorAdded(&InActor);

	// Loaded actors can be in sub-world partitions so we use the outer world
	if (UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(&InActor))
	{
		const FGuid& ActorGuid = InActor.GetActorGuid();
		if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid))
		{
			FSceneOutlinerHierarchyChangedData EventData;
			EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
			EventData.ItemIDs.Add(FActorDescTreeItem::ComputeTreeItemID(ActorDescInstance->GetGuid(), ActorDescInstance->GetContainerInstance()));
			HierarchyChangedEvent.Broadcast(EventData);
		}
	}
}

void FObjectMixerOutlinerHierarchy::OnLoadedActorRemoved(AActor& InActor)
{
	OnLevelActorDeleted(&InActor);
}

void FObjectMixerOutlinerHierarchy::OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* ActorDescInstance)
{
	if (bShowingUnloadedActors && ActorDescInstance)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
		EventData.ItemIDs.Add(FActorDescTreeItem::ComputeTreeItemID(ActorDescInstance->GetGuid(), ActorDescInstance->GetContainerInstance()));
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FObjectMixerOutlinerHierarchy::OnWorldPartitionInitialized(UWorldPartition* InWorldPartition)
{
	FullRefreshEvent();
}

void FObjectMixerOutlinerHierarchy::OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
{
	FullRefreshEvent();
}

void FObjectMixerOutlinerHierarchy::OnComponentsUpdated()
{
	FullRefreshEvent();
}

void FObjectMixerOutlinerHierarchy::OnLevelActorListChanged()
{
	FullRefreshEvent();
}

void FObjectMixerOutlinerHierarchy::OnActorFoldersUpdatedEvent(ULevel* InLevel)
{
	FullRefreshEvent();
}

void FObjectMixerOutlinerHierarchy::OnActorFolderAdded(UActorFolder* InActorFolder)
{
	check(InActorFolder);
	ULevel* Level = InActorFolder->GetOuterULevel();
	if (Level && Mode->ShouldShowFolders() && (RepresentingWorld.Get() == Level->GetWorld()))
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Added;
		EventData.Items.Add(Mode->CreateItemFor<FObjectMixerEditorListRowFolder>(FObjectMixerEditorListRowFolder(InActorFolder->GetFolder(), GetCastedMode()->GetSceneOutliner(), RepresentingWorld.Get())));
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FObjectMixerOutlinerHierarchy::OnLevelAdded(ULevel* InLevel, UWorld* InWorld)
{
	if (InLevel != nullptr && RepresentingWorld.Get() == InWorld)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Added;

		EventData.Items.Reserve(InLevel->Actors.Num());
		for (AActor* Actor : InLevel->Actors)
		{
			if (Actor != nullptr)
			{
				if (FSceneOutlinerTreeItemPtr ActorItem =
					Mode->CreateItemFor<FObjectMixerEditorListRowActor>(
						FObjectMixerEditorListRowActor(Actor, GetCastedMode()->GetSceneOutliner())))
				{
					EventData.Items.Add(ActorItem);
				}
			}
		}
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FObjectMixerOutlinerHierarchy::OnLevelRemoved(ULevel* InLevel, UWorld* InWorld)
{
	if (InLevel != nullptr && RepresentingWorld.Get() == InWorld)
	{
		{
			FSceneOutlinerHierarchyChangedData EventData;
			EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
			EventData.ItemIDs.Reserve(InLevel->Actors.Num());
			for (AActor* Actor : InLevel->Actors)
			{
				if (Actor != nullptr)
				{
					EventData.ItemIDs.Add(Actor);
				}
			}
			HierarchyChangedEvent.Broadcast(EventData);
		}

		// If either this level or the owning world are using actor folders, remove level's actor folders
		if (InLevel->IsUsingActorFolders() || InWorld->PersistentLevel->IsUsingActorFolders())
		{
			FSceneOutlinerHierarchyChangedData EventData;
			EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
			EventData.ItemIDs.Add(InLevel);
			HierarchyChangedEvent.Broadcast(EventData);
		}
	}
}

/** Called when a folder is to be created */
void FObjectMixerOutlinerHierarchy::OnBroadcastFolderCreate(UWorld& InWorld, const FFolder& InNewFolder)
{
	if (Mode->ShouldShowFolders() && RepresentingWorld.Get() == &InWorld)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Added;
		EventData.Items.Add(Mode->CreateItemFor<FObjectMixerEditorListRowFolder>(FObjectMixerEditorListRowFolder(InNewFolder, GetCastedMode()->GetSceneOutliner(), &InWorld)));
		EventData.ItemActions = SceneOutliner::ENewItemAction::Select | SceneOutliner::ENewItemAction::Rename;
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

/** Called when a folder is to be moved */
void FObjectMixerOutlinerHierarchy::OnBroadcastFolderMove(UWorld& InWorld, const FFolder& InOldFolder, const FFolder& InNewFolder)
{
	if (Mode->ShouldShowFolders() && RepresentingWorld.Get() == &InWorld)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::FolderMoved;
		EventData.ItemIDs.Add(InOldFolder);
		EventData.NewPaths.Add(InNewFolder);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

/** Called when a folder is to be deleted */
void FObjectMixerOutlinerHierarchy::OnBroadcastFolderDelete(UWorld& InWorld, const FFolder& InFolder)
{
	if (Mode->ShouldShowFolders() && RepresentingWorld.Get() == &InWorld)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
		EventData.ItemIDs.Add(InFolder);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

void FObjectMixerOutlinerHierarchy::OnLevelActorFolderChanged(const AActor* InActor, FName OldPath)
{
	if (Mode->ShouldShowFolders() && RepresentingWorld.Get() == InActor->GetWorld())
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Moved;
		EventData.ItemIDs.Add(FSceneOutlinerTreeItemID(InActor));
		HierarchyChangedEvent.Broadcast(EventData);
	}
}
