// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/Modes/ObjectMixerOutlinerMode.h"

#include "ObjectMixerEditorLog.h"
#include "ObjectMixerEditorModule.h"
#include "ObjectMixerEditorSettings.h"
#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"
#include "Views/List/ObjectMixerEditorList.h"
#include "Views/List/Modes/ObjectMixerOutlinerHierarchy.h"
#include "Views/List/Modes/SFilterClassMenuItem.h"
#include "Views/List/ObjectMixerUtils.h"
#include "Views/List/SObjectMixerEditorList.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowActor.h"
#include "Views/Widgets/ObjectMixerEditorListMenuContext.h"

#include "ActorDescTreeItem.h"
#include "ActorEditorUtils.h"
#include "ActorFolderPickingMode.h"
#include "ActorFolderTreeItem.h"
#include "ActorTreeItem.h"
#include "Blueprint/BlueprintSupport.h"
#include "ComponentTreeItem.h"
#include "EditorActorFolders.h"
#include "EditorLevelUtils.h"
#include "EditorViewportCommands.h"
#include "Engine/Blueprint.h"
#include "FolderTreeItem.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "IContentBrowserSingleton.h"
#include "ISourceControlModule.h"
#include "LevelEditor.h"
#include "LevelEditorContextMenu.h"
#include "LevelTreeItem.h"
#include "SceneOutlinerDelegates.h"
#include "SceneOutlinerMenuContext.h"
#include "SourceControlOperations.h"
#include "UnrealEdGlobals.h"
#include "WorldTreeItem.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "Editor/GroupActor.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Selection.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "LevelInstance/LevelInstanceEditorInstanceActor.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Logging/MessageLog.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Styling/StyleColors.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "ToolMenus.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowContainer.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowUObject.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectMixerOutlinerMode)

static int32 GSceneOutlinerAutoRepresentingWorldNetModeForObjectMixer = NM_Client;
static FAutoConsoleVariableRef CVarAutoRepresentingWorldNetMode(
	TEXT("SceneOutliner.AutoRepresentingWorldNetModeForObjectMixer"),
	GSceneOutlinerAutoRepresentingWorldNetModeForObjectMixer,
	TEXT("The preferred NetMode of the world shown in the scene outliner when the 'Auto' option is chosen: 0=Standalone, 1=DedicatedServer, 2=ListenServer, 3=Client"));

#define LOCTEXT_NAMESPACE "ObjectMixerEditor"

using FActorFilter = TSceneOutlinerPredicateFilter<FActorTreeItem>;
using FActorDescFilter = TSceneOutlinerPredicateFilter<FActorDescTreeItem>;
using FFolderFilter = TSceneOutlinerPredicateFilter<FFolderTreeItem>;

TObjectPtr<UObjectMixerOutlinerModeEditorConfig> UObjectMixerOutlinerModeEditorConfig::Instance = nullptr;

namespace ObjectMixerOutliner
{
	FHybridRowSelector::FHybridRowSelector(const FObjectMixerOutlinerMode* Mode)
	{
		check(Mode);
		bAllowHybridRows = Mode->ShouldAllowHybridRows();
	}

	bool FWeakActorSelectorAcceptingComponents::operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, TWeakObjectPtr<AActor>& DataOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FActorTreeItem* ActorItem = ItemPtr->CastTo<FActorTreeItem>())
			{
				if (ActorItem->IsValid())
				{
					DataOut = ActorItem->Actor;
					return true;
				}
			}
			// If a component is selected, we meant for the owning actor to be selected
			else if (FComponentTreeItem* ComponentItem = ItemPtr->CastTo<FComponentTreeItem>())
			{
				if (ComponentItem->IsValid())
				{
					AActor* Actor = ComponentItem->Component->GetOwner();
					if (Actor)
					{
						DataOut = Actor;
						return true;
					}
				}
			}
		}
		return false;
	}

	bool FComponentSelector::operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, UActorComponent*& DataOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FComponentTreeItem* ComponentTreeItem = ItemPtr->CastTo<FComponentTreeItem>())
			{
				if (ComponentTreeItem->IsValid())
				{
					DataOut = ComponentTreeItem->Component.Get();
					return true;
				}
			}
			else if (FObjectMixerEditorListRowActor* ActorItem = ItemPtr->CastTo<FObjectMixerEditorListRowActor>())
			{
				if (ActorItem->IsValid())
				{
					// Get the component from a hybrid row if we're in HybridComponent mode
					if (ShouldAllowHybridRows() && GetDefault<UObjectMixerEditorSettings>()->HybridRowPolicy == EObjectMixerHybridMode::HybridComponent &&
						ActorItem->RowData.GetHybridComponent())
					{
						DataOut = ActorItem->RowData.GetHybridComponent();
						return true;
					}
				}
			}
		}
		
		return false;
	}

	bool FWeakActorSelector::operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, TWeakObjectPtr<AActor>& DataOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FActorTreeItem* ActorItem = ItemPtr->CastTo<FActorTreeItem>())
			{
				if (ActorItem->IsValid())
				{
					DataOut = ActorItem->Actor;
					return true;
				}
			}
		}
		return false;
	}

	bool FActorSelector::operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, AActor*& ActorPtrOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FObjectMixerEditorListRowActor* ActorItem = ItemPtr->CastTo<FObjectMixerEditorListRowActor>())
			{
				if (ActorItem->IsValid())
				{
					// Skip hybrid rows here if in HybridComponent mode, we'll get the components from them in FComponentSelector
					if (ShouldAllowHybridRows() && GetDefault<UObjectMixerEditorSettings>()->HybridRowPolicy == EObjectMixerHybridMode::HybridComponent &&
						ActorItem->RowData.GetIsHybridRow())
					{
						return false;
					}
					
					AActor* Actor = ActorItem->Actor.Get();
					if (Actor)
					{
						ActorPtrOut = Actor;
						return true;
					}
				}
			}
			// If a component is selected, we meant for the owning actor to be selected
			else if (FComponentTreeItem* ComponentItem = ItemPtr->CastTo<FComponentTreeItem>())
			{
				if (ComponentItem->IsValid())
				{
					AActor* Actor = ComponentItem->Component->GetOwner();
					if (Actor)
					{
						ActorPtrOut = Actor;
						return true;
					}
				}
			}
			else if (FObjectMixerEditorListRowUObject* ObjectItem = ItemPtr->CastTo<FObjectMixerEditorListRowUObject>())
			{
				if (UObject* Object = ObjectItem->ObjectSoftPtr.Get())
				{
					if (AActor* Actor = Object->GetTypedOuter<AActor>())
					{
						ActorPtrOut = Actor;
						return true;
					}					
				}
			}
			else if (FObjectMixerEditorListRowContainer* ContainerItem = ItemPtr->CastTo<FObjectMixerEditorListRowContainer>())
			{
				if (UObject* PropertyOwner = ContainerItem->GetPropertyOwner())
				{
					if (AActor* Actor = PropertyOwner->GetTypedOuter<AActor>())
					{
						ActorPtrOut = Actor;
						return true;
					}
				}
			}
		}

		return false;
	}

	bool FActorHandleSelector::operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, FWorldPartitionHandle& ActorHandleOut) const
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FActorDescTreeItem* ActorDescItem = ItemPtr->CastTo<FActorDescTreeItem>())
			{
				ActorHandleOut = ActorDescItem->ActorDescHandle;
				return true;
			}
		}

		return false;
	}

	bool FFolderPathSelector::operator()(TWeakPtr<ISceneOutlinerTreeItem> Item, FFolder& DataOut) const
	{
		if (FFolderTreeItem* FolderItem = Item.Pin()->CastTo<FFolderTreeItem>())
		{
			if (FolderItem->IsValid())
			{
				DataOut = FolderItem->GetFolder();
				return true;
			}
		}
		return false;
	}

	bool FObjectSelector::operator()(TWeakPtr<ISceneOutlinerTreeItem> Item, UObject*& OutObject)
	{
		if (TSharedPtr<ISceneOutlinerTreeItem> ItemPtr = Item.Pin())
		{
			if (FObjectMixerEditorListRowActor* ActorItem = ItemPtr->CastTo<FObjectMixerEditorListRowActor>())
			{
				if (ActorItem->IsValid())
				{
					if (AActor* Actor = ActorItem->Actor.Get())
					{
						OutObject = Actor;
						return true;
					}
				}
			}
			// If a component is selected, we meant for the owning actor to be selected
			else if (FComponentTreeItem* ComponentItem = ItemPtr->CastTo<FComponentTreeItem>())
			{
				if (ComponentItem->IsValid())
				{
					if (UActorComponent* ActorComponent = ComponentItem->Component.Get())
					{
						OutObject = ActorComponent;
						return true;
					}
				}
			}
			else if (FObjectMixerEditorListRowUObject* ObjectItem = ItemPtr->CastTo<FObjectMixerEditorListRowUObject>())
			{
				if (UObject* Object = ObjectItem->ObjectSoftPtr.Get())
				{
					OutObject = Object;
					return true;			
				}
			}
			else if (FObjectMixerEditorListRowContainer* ContainerItem = ItemPtr->CastTo<FObjectMixerEditorListRowContainer>())
			{
				if (UObject* PropertyOwner = ContainerItem->GetPropertyOwner())
				{
					OutObject = PropertyOwner;
					return true;
				}
			}
		}
		
		return false;
	}
}

FObjectMixerOutlinerMode::FObjectMixerOutlinerMode(
	const FObjectMixerOutlinerModeParams& Params, const TSharedRef<FObjectMixerEditorList> InListModel)
	: FActorMode(Params)
{
	ListModelPtr = InListModel;
	
	WorldPartitionEditorModule = FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");

	// Capture selection changes of bones from mesh selection in fracture tools
	FSceneOutlinerDelegates::Get().OnComponentsUpdated.AddRaw(this, &FObjectMixerOutlinerMode::OnComponentsUpdated);

	GEngine->OnLevelActorDeleted().AddRaw(this, &FObjectMixerOutlinerMode::OnLevelActorDeleted);

	GEditor->OnSelectUnloadedActorsEvent().AddRaw(this, &FObjectMixerOutlinerMode::OnSelectUnloadedActors);

	UActorEditorContextSubsystem::Get()->OnActorEditorContextSubsystemChanged().AddRaw(this, &FObjectMixerOutlinerMode::OnActorEditorContextSubsystemChanged);

	FEditorDelegates::OnEditCutActorsBegin.AddRaw(this, &FObjectMixerOutlinerMode::OnEditCutActorsBegin);
	FEditorDelegates::OnEditCutActorsEnd.AddRaw(this, &FObjectMixerOutlinerMode::OnEditCutActorsEnd);
	FEditorDelegates::OnEditCopyActorsBegin.AddRaw(this, &FObjectMixerOutlinerMode::OnEditCopyActorsBegin);
	FEditorDelegates::OnEditCopyActorsEnd.AddRaw(this, &FObjectMixerOutlinerMode::OnEditCopyActorsEnd);
	FEditorDelegates::OnEditPasteActorsBegin.AddRaw(this, &FObjectMixerOutlinerMode::OnEditPasteActorsBegin);
	FEditorDelegates::OnEditPasteActorsEnd.AddRaw(this, &FObjectMixerOutlinerMode::OnEditPasteActorsEnd);
	FEditorDelegates::OnDuplicateActorsBegin.AddRaw(this, &FObjectMixerOutlinerMode::OnDuplicateActorsBegin);
	FEditorDelegates::OnDuplicateActorsEnd.AddRaw(this, &FObjectMixerOutlinerMode::OnDuplicateActorsEnd);
	FEditorDelegates::OnDeleteActorsBegin.AddRaw(this, &FObjectMixerOutlinerMode::OnDeleteActorsBegin);
	FEditorDelegates::OnDeleteActorsEnd.AddRaw(this, &FObjectMixerOutlinerMode::OnDeleteActorsEnd);

	UObjectMixerOutlinerModeEditorConfig::Initialize();
	GetMutableDefault<UObjectMixerOutlinerModeEditorConfig>()->LoadEditorConfig();

	// Get a MutableConfig here to force create a config for the current outliner if it doesn't exist
	const FObjectMixerOutlinerModeConfig* SavedSettings = GetMutableConfig();

	// Create a local struct to use the default values if this outliner doesn't want to save configs
	FObjectMixerOutlinerModeConfig LocalSettings;

	// If this outliner doesn't want to save config (OutlinerIdentifier is empty, use the defaults)
	if (SavedSettings)
	{
		LocalSettings = *SavedSettings;
	}

	// Get the OutlinerModule to register FilterInfos with the FilterInfoMap
	FSceneOutlinerFilterInfo ShowOnlySelectedActorsInfo(
		LOCTEXT("ToggleShowOnlySelected", "Only Selected"),
		LOCTEXT("ToggleShowOnlySelectedToolTip", "When enabled, only displays actors that are currently selected."),
		LocalSettings.bShowOnlySelectedActors,
		FCreateSceneOutlinerFilter::CreateStatic(&FObjectMixerOutlinerMode::CreateShowOnlySelectedActorsFilter)
	);
	ShowOnlySelectedActorsInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FObjectMixerOutlinerModeConfig* Settings = GetMutableConfig();
			if(Settings)
			{
				Settings->bShowOnlySelectedActors = bIsActive;
				SaveConfig();
			}
		});
	FilterInfoMap.Add(TEXT("ShowOnlySelectedActors"), ShowOnlySelectedActorsInfo);

	FSceneOutlinerFilterInfo HideTemporaryActorsInfo(
		LOCTEXT("ToggleHideTemporaryActors", "Hide Temporary Actors"),
		LOCTEXT("ToggleHideTemporaryActorsToolTip", "When enabled, hides temporary/run-time Actors."),
		LocalSettings.bHideTemporaryActors,
		FCreateSceneOutlinerFilter::CreateStatic(&FObjectMixerOutlinerMode::CreateHideTemporaryActorsFilter)
	);
	HideTemporaryActorsInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FObjectMixerOutlinerModeConfig* Settings = GetMutableConfig();
			if(Settings)
			{
				Settings->bHideTemporaryActors = bIsActive;
				SaveConfig();
			}
		});
	FilterInfoMap.Add(TEXT("HideTemporaryActors"), HideTemporaryActorsInfo);

	FSceneOutlinerFilterInfo OnlyCurrentLevelInfo(
		LOCTEXT("ToggleShowOnlyCurrentLevel", "Only in Current Level"),
		LOCTEXT("ToggleShowOnlyCurrentLevelToolTip", "When enabled, only shows Actors that are in the Current Level."),
		LocalSettings.bShowOnlyActorsInCurrentLevel,
		FCreateSceneOutlinerFilter::CreateStatic(&FObjectMixerOutlinerMode::CreateIsInCurrentLevelFilter)
	);
	OnlyCurrentLevelInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FObjectMixerOutlinerModeConfig* Settings = GetMutableConfig();
			if(Settings)
			{
				Settings->bShowOnlyActorsInCurrentLevel = bIsActive;
				SaveConfig();
			}
		});
	FilterInfoMap.Add(TEXT("ShowOnlyCurrentLevel"), OnlyCurrentLevelInfo);

	FSceneOutlinerFilterInfo OnlyCurrentDataLayersInfo(
		LOCTEXT("ToggleShowOnlyCurrentDataLayers", "Only in any Current Data Layers"),
		LOCTEXT("ToggleShowOnlyCurrentDataLayersToolTip", "When enabled, only shows Actors that are in any Current Data Layers."),
		LocalSettings.bShowOnlyActorsInCurrentDataLayers,
		FCreateSceneOutlinerFilter::CreateStatic(&FObjectMixerOutlinerMode::CreateIsInCurrentDataLayersFilter)
	);
	OnlyCurrentDataLayersInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FObjectMixerOutlinerModeConfig* Settings = GetMutableConfig();
			if (Settings)
			{
				Settings->bShowOnlyActorsInCurrentDataLayers = bIsActive;
				SaveConfig();
			}
		});
	FilterInfoMap.Add(TEXT("ShowOnlyCurrentDataLayers"), OnlyCurrentDataLayersInfo);

	// Add a filter for unloaded actors to properly reflect the bShowOnlyActorsInCurrentDataLayers flag.
	SceneOutliner->AddFilter(MakeShared<FActorDescFilter>(FActorDescTreeItem::FFilterPredicate::CreateLambda([this](const FWorldPartitionActorDescInstance* ActorDescInstance)
		{
			FObjectMixerOutlinerModeConfig* Settings = GetMutableConfig();
			if (Settings && Settings->bShowOnlyActorsInCurrentDataLayers)
			{
				return true;
			}
			return true;
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	FSceneOutlinerFilterInfo OnlyCurrentContentBundleInfo(
		LOCTEXT("ToggleShowOnlyCurrentContentBundle", "Only in Current Content Bundle"),
		LOCTEXT("ToggleShowOnlyCurrentContentBundleToolTip", "When enabled, only shows Actors that are in the Current Content Bundle."),
		LocalSettings.bShowOnlyActorsInCurrentContentBundle,
		FCreateSceneOutlinerFilter::CreateRaw(this, &FObjectMixerOutlinerMode::CreateIsInCurrentContentBundleFilter)
	);
	OnlyCurrentContentBundleInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FObjectMixerOutlinerModeConfig* Settings = GetMutableConfig();
			if (Settings)
			{
				Settings->bShowOnlyActorsInCurrentContentBundle = bIsActive;
				SaveConfig();
			}
		});
	FilterInfoMap.Add(TEXT("ShowOnlyCurrentContentBundle"), OnlyCurrentContentBundleInfo);

	// Add a filter for unloaded actors to properly reflect the bShowOnlyActorsInCurrentContentBundle flag.
	SceneOutliner->AddFilter(MakeShared<FActorDescFilter>(FActorDescTreeItem::FFilterPredicate::CreateLambda([this](const FWorldPartitionActorDescInstance* ActorDescInstance)
		{
			FObjectMixerOutlinerModeConfig* Settings = GetMutableConfig();
			if (Settings && Settings->bShowOnlyActorsInCurrentContentBundle)
			{
				if (!WorldPartitionEditorModule || !WorldPartitionEditorModule->IsEditingContentBundle())
				{
					return true;
				}

				return ActorDescInstance->GetContentBundleGuid().IsValid() && WorldPartitionEditorModule->IsEditingContentBundle(ActorDescInstance->GetContentBundleGuid());
			}
			return true;
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	bHideLevelInstanceHierarchy = LocalSettings.bHideLevelInstanceHierarchy;
	FSceneOutlinerFilterInfo HideLevelInstancesInfo(
		LOCTEXT("ToggleHideLevelInstanceContent", "Hide Level Instance Content"),
		LOCTEXT("ToggleHideLevelInstancesToolTip", "When enabled, hides all level instance content."),
		LocalSettings.bHideLevelInstanceHierarchy,
		FCreateSceneOutlinerFilter::CreateStatic(&FObjectMixerOutlinerMode::CreateHideLevelInstancesFilter)
	);
	HideLevelInstancesInfo.OnToggle().AddLambda([this](bool bIsActive)
		{
			FObjectMixerOutlinerModeConfig* Settings = GetMutableConfig();
			if(Settings)
			{
				Settings->bHideLevelInstanceHierarchy = bHideLevelInstanceHierarchy = bIsActive;
				SaveConfig();
			}

			if (auto ActorHierarchy = StaticCast<FObjectMixerOutlinerHierarchy*>(Hierarchy.Get()))
			{
				ActorHierarchy->SetShowingLevelInstances(!bIsActive);
			}
		});
	FilterInfoMap.Add(TEXT("HideLevelInstancesFilter"), HideLevelInstancesInfo);

	// Add a filter which sets the interactive mode of LevelInstance items and their children
	SceneOutliner->AddFilter(
		MakeShared<FActorFilter>(FActorTreeItem::FFilterPredicate::CreateStatic(
			[](const AActor* Actor) {return true; }),
			FSceneOutlinerFilter::EDefaultBehaviour::Pass,
			FActorTreeItem::FFilterPredicate::CreateLambda(
				[this](const AActor* Actor)
				{
					if (!bHideLevelInstanceHierarchy)
					{
						if (const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>())
						{
							// if actor has a valid parent and the parent is not being edited,
							// then the actor should not be selectable.
							if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
							{
								if (!LevelInstanceSubsystem->IsEditingLevelInstance(ParentLevelInstance))
								{
									return false;
								}
							}
						}
					}
					return true;
				}
			)
		)
	);

	bAlwaysFrameSelection = LocalSettings.bAlwaysFrameSelection;
	
	SceneOutliner->AddFilter(MakeShared<FActorFilter>(FActorTreeItem::FFilterPredicate::CreateLambda([this](const AActor* Actor)
	{
		return IsActorDisplayable(Actor);
	}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	auto FolderPassesFilter = [this](const FFolder& InFolder, bool bInCheckHideLevelInstanceFlag)
	{
		if (ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(InFolder.GetRootObjectPtr()))
		{
			if (LevelInstance->IsEditing())
			{
				return true;
			}
			if (bInCheckHideLevelInstanceFlag)
			{
				return !bHideLevelInstanceHierarchy;
			}
		}
		if (ULevel* Level = Cast<ULevel>(InFolder.GetRootObjectPtr()))
		{
			return true;
		}
		return false;
	};

	SceneOutliner->AddFilter(MakeShared<FFolderFilter>(FFolderTreeItem::FFilterPredicate::CreateLambda([FolderPassesFilter](const FFolder& InFolder)
	{
		return FolderPassesFilter(InFolder, /*bCheckHideLevelInstanceFlag*/true);
	}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	SceneOutliner->AddInteractiveFilter(MakeShared<FFolderFilter>(FFolderTreeItem::FFilterPredicate::CreateLambda([FolderPassesFilter](const FFolder& InFolder)
	{
		return FolderPassesFilter(InFolder, /*bCheckHideLevelInstanceFlag*/false);
	}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	// Filter class for Collection Buttons
	SceneOutliner->AddFilter(MakeShared<FActorFilter>(FActorTreeItem::FFilterPredicate::CreateLambda([this](const AActor* Actor)
	{
		if (const TSharedPtr<FObjectMixerEditorList> ListModel = GetListModelPtr().Pin())
		{
			const TSet<FName>& SelectedCollections = ListModel->GetSelectedCollections();

			if (SelectedCollections.Num() == 0 || (SelectedCollections.Num() == 1 && SelectedCollections.Find("All")))
			{
				// "All" collection selected
				return true;
			}
			
			for (const FName& CollectionName : SelectedCollections)
			{
				if (FObjectMixerUtils::IsObjectRefInCollection(CollectionName, Actor, ListModel))
				{
					return true;
				}

				const AActor* Parent = Actor->GetTypedOuter<AActor>();

				if (Parent && FObjectMixerUtils::IsObjectRefInCollection(CollectionName, Parent, ListModel))
				{
					return true;
				}
			}
		}

		return false;
	}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));

	if (TSharedPtr<FObjectMixerEditorList> PinnedList = GetListModelPtr().Pin())
	{
		if (PinnedList->GetSelectionInterface().IsValid())
		{
			PinnedList->GetSelectionInterface()->OnSelectionChanged().AddRaw(this, &FObjectMixerOutlinerMode::OnEditorSelectionChanged);
		}
	}

	FEditorDelegates::MapChange.AddRaw(this, &FObjectMixerOutlinerMode::OnMapChange);
	FEditorDelegates::NewCurrentLevel.AddRaw(this, &FObjectMixerOutlinerMode::OnNewCurrentLevel);

	FCoreDelegates::OnActorLabelChanged.AddRaw(this, &FObjectMixerOutlinerMode::OnActorLabelChanged);
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FObjectMixerOutlinerMode::OnObjectsReplaced);
	FCoreUObjectDelegates::PostLoadMapWithWorld.AddRaw(this, &FObjectMixerOutlinerMode::OnPostLoadMapWithWorld);
	GEngine->OnLevelActorRequestRename().AddRaw(this, &FObjectMixerOutlinerMode::OnLevelActorRequestsRename);

	FObjectMixerOutlinerMode::Rebuild();
}

FObjectMixerOutlinerMode::~FObjectMixerOutlinerMode()
{
	if (RepresentingWorld.IsValid())
	{
		if (UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			WorldPartition->OnActorDescInstanceRemovedEvent.RemoveAll(this);
		}
	}
	FSceneOutlinerDelegates::Get().OnComponentsUpdated.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().RemoveAll(this);
		GEngine->OnLevelActorRequestRename().RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->OnSelectUnloadedActorsEvent().RemoveAll(this);

		if (UActorEditorContextSubsystem* ActorEditorContextSubsystem = UActorEditorContextSubsystem::Get())
		{
			ActorEditorContextSubsystem->OnActorEditorContextSubsystemChanged().RemoveAll(this);
		}
	}

	if (TSharedPtr<FObjectMixerEditorList> PinnedList = GetListModelPtr().Pin())
	{
		if (PinnedList->GetSelectionInterface().IsValid())
		{
			PinnedList->GetSelectionInterface()->OnSelectionChanged().RemoveAll(this);
		}
	}

	FEditorDelegates::OnEditCutActorsBegin.RemoveAll(this);
	FEditorDelegates::OnEditCutActorsEnd.RemoveAll(this);
	FEditorDelegates::OnEditCopyActorsBegin.RemoveAll(this);
	FEditorDelegates::OnEditCopyActorsEnd.RemoveAll(this);
	FEditorDelegates::OnEditPasteActorsBegin.RemoveAll(this);
	FEditorDelegates::OnEditPasteActorsEnd.RemoveAll(this);
	FEditorDelegates::OnDuplicateActorsBegin.RemoveAll(this);
	FEditorDelegates::OnDuplicateActorsEnd.RemoveAll(this);
	FEditorDelegates::OnDeleteActorsBegin.RemoveAll(this);
	FEditorDelegates::OnDeleteActorsEnd.RemoveAll(this);

	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);

	FCoreDelegates::OnActorLabelChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);
}

void FObjectMixerOutlinerMode::OnMapChange(uint32 MapFlags)
{
	// Instruct the scene outliner to generate a new hierarchy
	SceneOutliner->FullRefresh();
}

void FObjectMixerOutlinerMode::OnNewCurrentLevel()
{
	// Instruct the scene outliner to generate a new hierarchy
	SceneOutliner->FullRefresh();
}

void FObjectMixerOutlinerMode::OnEditorSelectionChanged()
{
	if (!ShouldSyncSelectionFromEditor())
	{
		return;
	}

	const FSceneOutlinerFilterInfo* ShowOnlySelectedActorsFilter = FilterInfoMap.Find(TEXT("ShowOnlySelectedActors"));

	// Since there is no way to know which items were removed/added to a selection, we must force a full refresh to handle this
	if (ShowOnlySelectedActorsFilter && ShowOnlySelectedActorsFilter->IsFilterActive())
	{
		SceneOutliner->FullRefresh();
		return;
	}
	
	// If the SceneOutliner's reentrant flag is set, the selection change has already been handled in the outliner class
	if (!SceneOutliner->GetIsReentrant())
	{
		SceneOutliner->RefreshSelection();

		// Scroll last item into view - this means if we are multi-selecting, we show newest selection. @TODO Not perfect though
		const TArray<AActor*> SelectedActors = GetSelectedActorsInEditor();
		if (!SelectedActors.IsEmpty())
		{
			if (const AActor* LastSelectedActor = SelectedActors.Last())
			{
				if (FSceneOutlinerTreeItemPtr TreeItem = SceneOutliner->GetTreeItem(LastSelectedActor, false))
				{
					// Only scroll if selection framing is enabled
					if (bAlwaysFrameSelection)
					{
						SceneOutliner->ScrollItemIntoView(TreeItem);
					}
				}
				else
				{
					SceneOutliner::ENewItemAction::Type Action = bAlwaysFrameSelection ? SceneOutliner::ENewItemAction::ScrollIntoView : SceneOutliner::ENewItemAction::Select;

					SceneOutliner->OnItemAdded(LastSelectedActor, Action);
				}
			}
		}
	}
}

void FObjectMixerOutlinerMode::OnLevelActorRequestsRename(const AActor* Actor)
{
	TWeakPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();

	if(TSharedPtr<ILevelEditor> LevelEditorPin = LevelEditor.Pin())
	{
		/* We want to execute the rename on the most recently used outliner
		 * TODO: Add a way to pop-out the outliner the rename is done on
		 */
		if(SceneOutliner == LevelEditorPin->GetMostRecentlyUsedSceneOutliner().Get())
		{
			const TArray<FSceneOutlinerTreeItemPtr>& SelectedItems = SceneOutliner->GetSelectedItems();
			if (SelectedItems.Num() > 0)
			{
				// Ensure that the item we want to rename is visible in the tree
				FSceneOutlinerTreeItemPtr ItemToRename = SelectedItems[SelectedItems.Num() - 1];
				if (SceneOutliner->CanExecuteRenameRequest(*ItemToRename) && ItemToRename->CanInteract())
				{
					SceneOutliner->SetPendingRenameItem(ItemToRename);
					SceneOutliner->ScrollItemIntoView(ItemToRename);
				}
			}
		}
	}
}

void FObjectMixerOutlinerMode::OnPostLoadMapWithWorld(UWorld* World)
{
	SceneOutliner->FullRefresh();
}

bool EvaluateSyncSelectionSetting()
{
	const bool bIsSyncSettingOn = GetDefault<UObjectMixerEditorSettings>()->bSyncSelection;
	const bool bIsAltDown = FSlateApplication::Get().GetModifierKeys().IsAltDown();

	// Alt key should invert the user setting
	return (bIsSyncSettingOn && !bIsAltDown) || (!bIsSyncSettingOn && bIsAltDown);
}

bool FObjectMixerOutlinerMode::ShouldSyncSelectionToEditor()
{
	return bShouldTemporarilyForceSelectionSyncToEditor || EvaluateSyncSelectionSetting();
}

bool FObjectMixerOutlinerMode::ShouldSyncSelectionFromEditor()
{
	return bShouldTemporarilyForceSelectionSyncFromEditor || EvaluateSyncSelectionSetting();
}

void FObjectMixerOutlinerMode::SynchronizeAllSelectionsToEditor()
{
	bool bAreAnyInPIE = false;

	// Actors
	SynchronizeSelectedActorDescs();
	TArray<AActor*> SelectedActors = SceneOutliner->GetSelection().GetData<AActor*>(ObjectMixerOutliner::FActorSelector(this));
	bool bHasActorSelectionChanged = HasActorSelectionChanged(SelectedActors, bAreAnyInPIE);
		
	// Components
	TArray<UActorComponent*> SelectedComponents = SceneOutliner->GetSelection().GetData<UActorComponent*>(ObjectMixerOutliner::FComponentSelector(this));
	bool bHasComponentSelectionChanged = HasComponentSelectionChanged(SelectedComponents, bAreAnyInPIE);
		
	// If there's a discrepancy, update the selected objects to reflect the list.
	if (bHasActorSelectionChanged || bHasComponentSelectionChanged)
	{
		const bool bShouldActuallyTransact = !bAreAnyInPIE;
		const FScopedTransaction Transaction(LOCTEXT("ClickingOnActorsAndComponents", "Clicking on Actors & Components"), bShouldActuallyTransact);

		if (bHasActorSelectionChanged)
		{
			SelectActorsInEditor(SelectedActors, true, true);
		}
			
		if (bHasComponentSelectionChanged)
		{
			SelectComponentsInEditor(SelectedComponents, true, true);
		}
	}

	TArray<UObject*> SelectedObjects = SceneOutliner->GetSelection().GetData<UObject*>(ObjectMixerOutliner::FObjectSelector());
	SelectSubObjectsInEditor(SelectedObjects, true, true);	
}

bool FObjectMixerOutlinerMode::HasActorSelectionChanged(TArray<AActor*>& OutSelectedActors, bool& bOutAreAnyInPIE)
{
	TArray<AActor*> EditorSelectedActors = GetSelectedActorsInEditor();

	bool bHasSelectionChanged = false;
	for (AActor* Actor : OutSelectedActors)
	{
		if (!bOutAreAnyInPIE && Actor && Actor->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor))
		{
			bOutAreAnyInPIE = true;
		}
		if (!EditorSelectedActors.Contains(Actor))
		{
			bHasSelectionChanged = true;
			break;
		}
	}

	for (AActor* Actor : EditorSelectedActors)
	{
		if (!bOutAreAnyInPIE && Actor->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor))
		{
			bOutAreAnyInPIE = true;
		}
		if (!OutSelectedActors.Contains(Actor))
		{
			// Actor has been deselected
			bHasSelectionChanged = true;

			// If actor was a group actor, remove its members from the ActorsToSelect list
			const AGroupActor* DeselectedGroupActor = Cast<AGroupActor>(Actor);
			if (DeselectedGroupActor)
			{
				TArray<AActor*> GroupActors;
				DeselectedGroupActor->GetGroupActors(GroupActors);

				for (auto* GroupActor : GroupActors)
				{
					OutSelectedActors.Remove(GroupActor);
				}
			}
		}

		if (bHasSelectionChanged)
		{
			break;
		}
	}

	return bHasSelectionChanged;
}

bool FObjectMixerOutlinerMode::HasComponentSelectionChanged(TArray<UActorComponent*>& OutSelectedComponents, bool& bOutAreAnyInPIE)
{
	TArray<UActorComponent*> EditorSelectedComponents = GetSelectedComponentsInEditor();

	bool bHasSelectionChanged = false;
	for (UActorComponent* Component : OutSelectedComponents)
	{
		if (!bOutAreAnyInPIE && Component && Component->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor))
		{
			bOutAreAnyInPIE = true;
		}
		if (!EditorSelectedComponents.Contains(Component))
		{
			bHasSelectionChanged = true;
			break;
		}
	}

	for (UActorComponent* Component : EditorSelectedComponents)
	{
		if (!bOutAreAnyInPIE && Component->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor))
		{
			bOutAreAnyInPIE = true;
		}
		if (!OutSelectedComponents.Contains(Component))
		{
			// Component has been deselected
			bHasSelectionChanged = true;
			break;
		}
	}

	return bHasSelectionChanged;
}

void FObjectMixerOutlinerMode::SelectActorsInEditor(const TArray<AActor*>& InSelectedActors, bool bShouldSelect, bool bSelectEvenIfHidden)
{
	if (TSharedPtr<FObjectMixerEditorList> PinnedList = GetListModelPtr().Pin())
	{
		if (PinnedList->GetSelectionInterface().IsValid())
		{
			PinnedList->GetSelectionInterface()->SelectActors(InSelectedActors, bShouldSelect, bSelectEvenIfHidden);
		}
	}
}

void FObjectMixerOutlinerMode::SelectComponentsInEditor(const TArray<UActorComponent*>& InSelectedComponents, bool bShouldSelect, bool bSelectEvenIfHidden)
{
	if (TSharedPtr<FObjectMixerEditorList> PinnedList = GetListModelPtr().Pin())
	{
		if (PinnedList->GetSelectionInterface().IsValid())
		{
			PinnedList->GetSelectionInterface()->SelectComponents(InSelectedComponents, bShouldSelect, bSelectEvenIfHidden);
		}
	}
}

void FObjectMixerOutlinerMode::SelectSubObjectsInEditor(const TArray<UObject*>& InSelectedSubObjects, bool bShouldSelect, bool bSelectEvenIfHidden)
{
	if (TSharedPtr<FObjectMixerEditorList> PinnedList = GetListModelPtr().Pin())
	{
		if (PinnedList->GetSelectionInterface().IsValid())
		{
			PinnedList->GetSelectionInterface()->SelectSubObjects(InSelectedSubObjects, bShouldSelect, bSelectEvenIfHidden);
		}
	}
}

void FObjectMixerOutlinerMode::SelectActorsInMixer(const TArray<AActor*>& InSelectedActors, bool bShouldSelect, bool bSelectEvenIfHidden)
{
	TArray<FSceneOutlinerTreeItemPtr> ItemsToSelect;
	ItemsToSelect.Reserve(InSelectedActors.Num());

	Algo::TransformIf(InSelectedActors, ItemsToSelect,
		[this, bSelectEvenIfHidden](AActor* Actor) -> bool
		{
			if (const FSceneOutlinerTreeItemPtr& Item = SceneOutliner->GetTreeItem(Actor))
			{
				return bSelectEvenIfHidden || !Item->Flags.bIsFilteredOut;
			}

			return false;
		},
		[this](AActor* Actor) -> FSceneOutlinerTreeItemPtr
		{
			return SceneOutliner->GetTreeItem(Actor);
		}
	);

	SceneOutliner->SetItemSelection(ItemsToSelect, bShouldSelect);
}

void FObjectMixerOutlinerMode::SelectComponentsInMixer(const TArray<UActorComponent*>& InSelectedComponents, bool bShouldSelect, bool bSelectEvenIfHidden)
{
	TArray<FSceneOutlinerTreeItemPtr> ItemsToSelect;
	ItemsToSelect.Reserve(InSelectedComponents.Num());

	Algo::TransformIf(InSelectedComponents, ItemsToSelect,
		[this, bSelectEvenIfHidden](UActorComponent* Component) -> bool
		{
			if (const FSceneOutlinerTreeItemPtr& Item = SceneOutliner->GetTreeItem(Component))
			{
				return bSelectEvenIfHidden || !Item->Flags.bIsFilteredOut;
			}

			return false;
		},
		[this](UActorComponent* Component) -> FSceneOutlinerTreeItemPtr
		{
			return SceneOutliner->GetTreeItem(Component);
		}
	);

	SceneOutliner->SetItemSelection(ItemsToSelect, bShouldSelect);
}

TArray<AActor*> FObjectMixerOutlinerMode::GetSelectedActorsInEditor() const
{
	if (TSharedPtr<FObjectMixerEditorList> PinnedList = GetListModelPtr().Pin())
	{
		if (PinnedList->GetSelectionInterface().IsValid())
		{
			return PinnedList->GetSelectionInterface()->GetSelectedActors();
		}
	}

	return TArray<AActor*>();
}

TArray<UActorComponent*> FObjectMixerOutlinerMode::GetSelectedComponentsInEditor() const
{
	if (TSharedPtr<FObjectMixerEditorList> PinnedList = GetListModelPtr().Pin())
	{
		if (PinnedList->GetSelectionInterface().IsValid())
		{
			return PinnedList->GetSelectionInterface()->GetSelectedComponents();
		}
	}

	return TArray<UActorComponent*>();
}

TArray<UObject*> FObjectMixerOutlinerMode::GetSelectedSubObjectsInEditor() const
{
	if (TSharedPtr<FObjectMixerEditorList> PinnedList = GetListModelPtr().Pin())
	{
		if (PinnedList->GetSelectionInterface().IsValid())
		{
			return PinnedList->GetSelectionInterface()->GetSelectedSubObjects();
		}
	}

	return TArray<UObject*>();
}

bool FObjectMixerOutlinerMode::GetIsObjectOfFilteredClass(const UObject* Object) const
{
	if (IsValid(Object))
	{
		for (UClass* Class : GetListModelPtr().Pin()->ObjectClassesToFilterCache)
		{
			if (Object->IsA(Class))
			{
				return true;
			}
		}
	}

	return false;
}

void FObjectMixerOutlinerMode::OnActorLabelChanged(AActor* ChangedActor)
{
	if (!ensure(ChangedActor))
	{
		return;
	}

	if (!IsActorDisplayable(ChangedActor) || RepresentingWorld.Get() != ChangedActor->GetWorld())
	{
		return;
	}

	// Check if this actor is one of the registered types
	if (GetIsObjectOfFilteredClass(ChangedActor))
	{
		// Force create the item otherwise the outliner may not be notified of a change to the item if it is filtered out
		if (FSceneOutlinerTreeItemPtr Item = CreateItemFor<FObjectMixerEditorListRowActor>(
			FObjectMixerEditorListRowActor(ChangedActor, GetSceneOutliner()), true))
		{
			SceneOutliner->OnItemLabelChanged(Item);
		}
	}
	else if (const UObjectMixerObjectFilter* Filter = GetListModelPtr().Pin()->GetMainObjectFilterInstance())
	{
		// This actor is likely an associated actor of one or more registered type actors, so we need to force an update for each parent/associated pair
		const TSet<TWeakObjectPtr<AActor>> ApplicableActorsCopy = ApplicableActors; // This set may change while we're iterating, so make a copy
		for (TWeakObjectPtr<AActor> ApplicableActor : ApplicableActorsCopy)
		{
			if (!ApplicableActor.IsValid())
			{
				continue;
			}

			if (Filter->IsActorAssociated(ApplicableActor.Get(), ChangedActor))
			{
				if (FSceneOutlinerTreeItemPtr Item = CreateItemFor<FObjectMixerEditorListRowActor>(
					FObjectMixerEditorListRowActor(ChangedActor, GetSceneOutliner(), FText::GetEmpty(), ApplicableActor.Get()), true))
				{
					SceneOutliner->OnItemLabelChanged(Item);
				}
			}
		}
	}
}

void FObjectMixerOutlinerMode::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	for (const TPair<UObject*, UObject*>& Pair : ReplacementMap)
	{
		AActor* Actor = Cast<AActor>(Pair.Value);
		if (Actor && RepresentingWorld.Get() == Actor->GetWorld() && IsActorDisplayable(Actor))
		{
			if (FSceneOutlinerTreeItemPtr Item = CreateItemFor<FObjectMixerEditorListRowActor>(
				FObjectMixerEditorListRowActor(Actor, GetSceneOutliner()), true))
			{
				SceneOutliner->OnItemLabelChanged(Item);
			}
		}

		const int32 SelectionIndex = SelectedIDs.IndexOfByKey(Pair.Key);
		if (SelectionIndex != INDEX_NONE)
		{
			SelectedIDs[SelectionIndex] = Pair.Value;
		}
	}
}

UWorld* FObjectMixerOutlinerMode::GetRepresentingWorld()
{
	// Select a world to represent

	RepresentingWorld = nullptr;

	// If a specified world was provided, represent it
	if (SpecifiedWorldToDisplay.IsValid())
	{
		RepresentingWorld = SpecifiedWorldToDisplay.Get();
	}

	// check if the user-chosen world is valid and in the editor contexts

	if (!RepresentingWorld.IsValid() && UserChosenWorld.IsValid())
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (UserChosenWorld.Get() == Context.World())
			{
				RepresentingWorld = UserChosenWorld.Get();
				break;
			}
		}
	}

	// If the user did not manually select a world, try to pick the most suitable world context
	if (!RepresentingWorld.IsValid())
	{
		// Ideally we want a PIE world that is standalone or the first client, unless the preferred NetMode is overridden by CVar
		int32 LowestPIEInstanceSeen = MAX_int32;
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World && Context.WorldType == EWorldType::PIE)
			{
				if (World->GetNetMode() == NM_Standalone)
				{
					RepresentingWorld = World;
					break;
				}
				else if ((World->GetNetMode() == ENetMode(GSceneOutlinerAutoRepresentingWorldNetModeForObjectMixer)) && (Context.PIEInstance < LowestPIEInstanceSeen))
				{
					RepresentingWorld = World;
					LowestPIEInstanceSeen = Context.PIEInstance;
				}
			}
		}
	}

	if (RepresentingWorld == nullptr)
	{
		// still no world so fallback to old logic where we just prefer PIE over Editor

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE)
			{
				RepresentingWorld = Context.World();
				break;
			}
			else if (Context.WorldType == EWorldType::Editor)
			{
				RepresentingWorld = Context.World();
			}
		}
	}

	return RepresentingWorld.Get();
}

TUniquePtr<ISceneOutlinerHierarchy> FObjectMixerOutlinerMode::CreateHierarchy()
{	
	TUniquePtr<FObjectMixerOutlinerHierarchy> OutlinerHierarchy = FObjectMixerOutlinerHierarchy::Create(this, RepresentingWorld);
	OutlinerHierarchy->SetShowingComponents(!bHideComponents);
	OutlinerHierarchy->SetShowingOnlyActorWithValidComponents(!bHideComponents && bHideActorWithNoComponent);
	OutlinerHierarchy->SetShowingLevelInstances(!bHideLevelInstanceHierarchy);
	OutlinerHierarchy->SetShowingUnloadedActors(!bHideUnloadedActors);
	OutlinerHierarchy->SetShowingEmptyFolders(!bHideEmptyFolders);

	return OutlinerHierarchy;
}

void FObjectMixerOutlinerMode::Rebuild()
{
	// If we used to be representing a wp world, unbind delegates before rebuilding begins
	if (RepresentingWorld.IsValid())
	{
		if (UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			WorldPartition->OnActorDescInstanceRemovedEvent.RemoveAll(this);
		}
	}

	GetRepresentingWorld();
	Hierarchy = CreateHierarchy();

	FilteredActorCount = 0;
	FilteredUnloadedActorCount = 0;
	ApplicableUnloadedActors.Empty();
	ApplicableActors.Empty();

	bRepresentingWorldGameWorld = RepresentingWorld.IsValid() && RepresentingWorld->IsGameWorld();
	bRepresentingWorldPartitionedWorld = RepresentingWorld.IsValid() && RepresentingWorld->IsPartitionedWorld();

	if (bRepresentingWorldPartitionedWorld)
	{
		UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition();
		WorldPartition->OnActorDescInstanceRemovedEvent.AddRaw(this, &FObjectMixerOutlinerMode::OnActorDescInstanceRemoved);
	}

	SceneOutliner->FullRefresh();
}

int32 FObjectMixerOutlinerMode::GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const
{
	if (Item.IsA<FWorldTreeItem>())
	{
		return EItemSortOrder::World;
	}
	else if (Item.IsA<FLevelTreeItem>())
	{
		return EItemSortOrder::Level;
	}
	else if (Item.IsA<FFolderTreeItem>())
	{
		return EItemSortOrder::Folder;
	}
	else if (Item.IsA<FActorTreeItem>() || Item.IsA<FComponentTreeItem>() || Item.IsA<FActorDescTreeItem>())
	{
		return EItemSortOrder::Actor;
	}

	return 40;
}

FText FObjectMixerOutlinerMode::GetStatusText() const
{
	if (!RepresentingWorld.IsValid())
	{
		return FText();
	}

	// The number of actors in the outliner before applying the text filter
	const int32 TotalActorCount = ApplicableActors.Num() + ApplicableUnloadedActors.Num();
	const int32 SelectedActorCount = SceneOutliner->GetSelection().Num<FActorTreeItem, FActorDescTreeItem>();

	if (!SceneOutliner->IsTextFilterActive())
	{
		if (SelectedActorCount == 0)
		{
			if (bRepresentingWorldPartitionedWorld)
			{
				return FText::Format(LOCTEXT("ShowingAllLoadedActorsFmt", "{0} actors ({1} loaded)"), FText::AsNumber(FilteredActorCount), FText::AsNumber(FilteredActorCount - FilteredUnloadedActorCount));
			}
			else
			{
				return FText::Format(LOCTEXT("ShowingAllActorsFmt", "{0} actors"), FText::AsNumber(FilteredActorCount));
			}
		}
		else
		{
			return FText::Format(LOCTEXT("ShowingAllActorsSelectedFmt", "{0} actors ({1} selected)"), FText::AsNumber(FilteredActorCount), FText::AsNumber(SelectedActorCount));
		}
	}
	else if (SceneOutliner->IsTextFilterActive() && FilteredActorCount == 0)
	{
		return FText::Format(LOCTEXT("ShowingNoActorsFmt", "No matching actors ({0} total)"), FText::AsNumber(TotalActorCount));
	}
	else if (SelectedActorCount != 0)
	{
		return FText::Format(LOCTEXT("ShowingOnlySomeActorsSelectedFmt", "Showing {0} of {1} actors ({2} selected)"), FText::AsNumber(FilteredActorCount), FText::AsNumber(TotalActorCount), FText::AsNumber(SelectedActorCount));
	}
	else
	{
		return FText::Format(LOCTEXT("ShowingOnlySomeActorsFmt", "Showing {0} of {1} actors"), FText::AsNumber(FilteredActorCount), FText::AsNumber(TotalActorCount));
	}
}

FSlateColor FObjectMixerOutlinerMode::GetStatusTextColor() const
{
	if (!SceneOutliner->IsTextFilterActive())
	{
		return FSlateColor::UseForeground();
	}
	else if (FilteredActorCount == 0)
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	else
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
	}
}

void FObjectMixerOutlinerMode::OnActorEditorContextSubsystemChanged()
{
	SceneOutliner->FullRefresh();
}

void FObjectMixerOutlinerMode::OnToggleAlwaysFrameSelection()
{
	bAlwaysFrameSelection = !bAlwaysFrameSelection;

	FObjectMixerOutlinerModeConfig* Settings = GetMutableConfig();
	if(Settings)
	{
		Settings->bAlwaysFrameSelection = bAlwaysFrameSelection;
		SaveConfig();
	}
}

bool FObjectMixerOutlinerMode::ShouldAlwaysFrameSelection()
{
	return bAlwaysFrameSelection;
}

bool IsBlueprintFilter(const FAssetData& BlueprintClassData)
{
	UClass* BlueprintFilterClass = UObjectMixerBlueprintObjectFilter::StaticClass();
		
	const FString NativeParentClassPath = BlueprintClassData.GetTagValueRef<FString>(FBlueprintTags::NativeParentClassPath);
	const FSoftClassPath ClassPath(NativeParentClassPath);
			
	UClass* NativeParentClass = ClassPath.ResolveClass();
	const bool bInheritsFromBlueprintFilter =
		NativeParentClass // Class may have been removed, or renamed and not correctly redirected
		&& (NativeParentClass == BlueprintFilterClass || NativeParentClass->IsChildOf(BlueprintFilterClass));

	return bInheritsFromBlueprintFilter;
}

TSharedRef<SWidget> FObjectMixerOutlinerMode::OnGenerateFilterClassMenu()
{
	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	TSharedRef<SBox> OuterBox =
		SNew(SBox)
		.Padding(8)
		[
			VerticalBox
		];

	// Get C++ Derivatives (and maybe Blueprint derivatives)
	TArray<UClass*> DerivedClasses;
	GetDerivedClasses(UObjectMixerObjectFilter::StaticClass(), DerivedClasses, true);

	DerivedClasses.Remove(UObjectMixerObjectFilter::StaticClass());
	DerivedClasses.Remove(UObjectMixerBlueprintObjectFilter::StaticClass());

	TArray<FAssetClassMap> AssetClassMaps;

	for (UClass* Class : DerivedClasses)
	{
		AssetClassMaps.Add({Class});
	}

	// Get remaining Blueprint derivatives
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked< FAssetRegistryModule >(FName("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	TArray< FAssetData > Assets;
	AssetRegistry.GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), Assets);	
	for (const FAssetData& Asset : Assets)
	{
		if (IsBlueprintFilter(Asset))
		{
			if (const UBlueprint* BlueprintAsset = Cast<UBlueprint>(Asset.GetAsset()))
			{
				UClass* LoadedClass = BlueprintAsset->GeneratedClass;
				if (ensure(LoadedClass && BlueprintAsset->ParentClass))
				{
					if (FAssetClassMap* Match = Algo::FindByPredicate(
						AssetClassMaps,
						[LoadedClass](const FAssetClassMap& ClassMap)
						{
							return ClassMap.Class == LoadedClass;
						}))
					{
						Match->AssetData = Asset;
					}
					else
					{
						AssetClassMaps.Add({LoadedClass, Asset});
					}
				}
			}
		}
	}

	if (AssetClassMaps.Num())
	{
		check(GetListModelPtr().IsValid());

		AssetClassMaps.Sort([](const FAssetClassMap& A, const FAssetClassMap& B)
		{
			return A.Class.GetFName().LexicalLess(B.Class.GetFName());
		});
		
		FilterClassSelectionInfos.Empty(FilterClassSelectionInfos.Num());
		for (const FAssetClassMap& AssetClassMap : AssetClassMaps)
		{
			if (IsValid(AssetClassMap.Class))
			{
				if (AssetClassMap.Class->GetName().StartsWith(TEXT("SKEL_")) || AssetClassMap.Class->GetName().StartsWith(TEXT("REINST_")))
				{
					continue;
				}

				if (AssetClassMap.Class->HasAnyClassFlags(CLASS_Abstract | CLASS_HideDropDown | CLASS_Deprecated))
				{
					continue;
				}

				TSharedPtr<FObjectMixerEditorList> ListModel = GetListModelPtr().Pin();

				const bool bIsDefaultClass =
					// If this is a made-to-purpose sub-plugin of Object Mixer, don't allow default class to be disabled
					ListModel->GetModuleName() != FObjectMixerEditorModule::BaseObjectMixerModuleName &&
					ListModel->GetDefaultFilterClass() == AssetClassMap.Class;
				
				const FText TooltipText = bIsDefaultClass ?
					FText::Format(
						LOCTEXT("DefaultClassDisclaimer","This class explicitly cannot be disabled in {0}"), 
						FText::FromName(ListModel->GetModuleName())) :
					FText::FromString(AssetClassMap.Class->GetClassPathName().ToString()
				);

				FilterClassSelectionInfos.Add({AssetClassMap.Class, ListModel->IsClassSelected(AssetClassMap.Class)});

				VerticalBox->AddSlot()
				.Padding(FMargin(0, 0, 0, 8))
				.AutoHeight()
				[
					SNew(SFilterClassMenuItem, AssetClassMap, bIsDefaultClass, FilterClassSelectionInfos, TooltipText)
				];
			}
		}

		VerticalBox->AddSlot()
		.HAlign(HAlign_Fill)
		[
			SNew(SButton)
			.Text(LOCTEXT("SelectFilterClassMenu_ApplyButton", "Apply"))
			.HAlign(HAlign_Center)
			.OnClicked(FOnClicked::CreateLambda([this]()
			   {
					if (const TSharedPtr<FObjectMixerEditorList> PinnedList = GetListModelPtr().Pin())
					{
						PinnedList->ResetObjectFilterClasses(false);
						for (const FFilterClassSelectionInfo& Info : FilterClassSelectionInfos)
						{
							if (Info.bIsUserSelected)
							{
								PinnedList->AddObjectFilterClass(Info.Class, false);
							}
						}

						PinnedList->CacheAndRebuildFilters(true);
					}

					return FReply::Handled();
			   })
			)
		];
	}
	else
	{
		VerticalBox->AddSlot()
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NoFilterClassesAvailable", "No filter classes available."))
		];
	}

	FChildren* ChildWidgets = VerticalBox->GetChildren();
	for (int32 ChildItr = 0; ChildItr < ChildWidgets->Num(); ChildItr++)
	{
		const TSharedRef<SWidget>& Child = ChildWidgets->GetChildAt(ChildItr);

		Child->EnableToolTipForceField(false);
	}
	VerticalBox->EnableToolTipForceField(false);
	OuterBox->EnableToolTipForceField(false);
	
	return OuterBox;
}

EObjectMixerTreeViewMode FObjectMixerOutlinerMode::GetTreeViewMode() const
{
	const TSharedPtr<FObjectMixerEditorList> PinnedListModel = GetListModelPtr().Pin();
	check(PinnedListModel);
	
	return PinnedListModel->GetTreeViewMode();
}

void FObjectMixerOutlinerMode::SetTreeViewMode(EObjectMixerTreeViewMode InViewMode)
{
	if (TSharedPtr<FObjectMixerEditorList> PinnedListModel = GetListModelPtr().Pin())
	{
		PinnedListModel->SetTreeViewMode(InViewMode);
	}
}

void FObjectMixerOutlinerMode::CreateViewContent(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("OutlinerSelectionOptions", LOCTEXT("OptionsHeading", "Options"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("AlwaysFrameSelectionLabel", "Always Frame Selection"),
		LOCTEXT("AlwaysFrameSelectionTooltip", "When enabled, selecting an Actor in the Viewport also scrolls to that Actor in the Outliner."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateRaw(this, &FObjectMixerOutlinerMode::OnToggleAlwaysFrameSelection),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &FObjectMixerOutlinerMode::ShouldAlwaysFrameSelection)
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	MenuBuilder.EndSection();

	// No need to select filter class from outside generic instance
	MenuBuilder.BeginSection("ListViewOptions", LOCTEXT("FilterClassManagementSection", "Filter Class Management"));
	{
		// Filter Class Management Button
		const TSharedRef<SWidget> FilterClassManagementButton =
			SNew(SBox)
			.Padding(8, 0)
			[
				SNew(SComboButton)
				.ToolTipText(LOCTEXT("FilterClassManagementButton_Tooltip", "Select a filter class"))
				.ContentPadding(FMargin(4, 0.5f))
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton"))
				.OnGetMenuContent_Raw(this, &FObjectMixerOutlinerMode::OnGenerateFilterClassMenu)
				.ForegroundColor(FStyleColors::Foreground)
				.MenuPlacement(EMenuPlacement::MenuPlacement_MenuRight)
				.ButtonContent()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.Padding(0, 1, 4, 0)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]

					+ SHorizontalBox::Slot()
					.Padding(0, 1, 0, 0)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FilterClassToolbarButton", "Object Filter Class"))
					]
				]
			];

		MenuBuilder.AddWidget(FilterClassManagementButton, FText::GetEmpty());
	}
	MenuBuilder.EndSection();

	// Add List View Mode Options
	MenuBuilder.BeginSection("ListViewOptions", LOCTEXT("ListViewOptionsSection", "List View Options"));
	{
		// Foreach on uenum
		const FString EnumPath = "/Script/ObjectMixerEditor.EObjectMixerTreeViewMode";
		if (const UEnum* EnumPtr = FindObject<UEnum>(nullptr, *EnumPath, EFindObjectFlags::ExactClass))
		{
			for (int32 EnumItr = 0; EnumItr < EnumPtr->GetMaxEnumValue(); EnumItr++)
			{
				EObjectMixerTreeViewMode EnumValue = (EObjectMixerTreeViewMode)EnumItr;
				
				MenuBuilder.AddMenuEntry(
					EnumPtr->GetDisplayNameTextByIndex(EnumItr),
					EnumPtr->GetToolTipTextByIndex(EnumItr),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this, EnumValue]()
						{
							SetTreeViewMode(EnumValue);
						}),
						FCanExecuteAction::CreateLambda([](){ return true; }),
						FIsActionChecked::CreateLambda([this, EnumValue]()
						{
							return GetTreeViewMode() == EnumValue;
						})
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("MiscOptionsSection", LOCTEXT("MiscOptionsSection","Misc"));
	{
		// No need to open generic instance from itself
		if (GetListModelPtr().Pin()->GetModuleName() != FObjectMixerEditorModule::BaseObjectMixerModuleName)
		{
			MenuBuilder.AddMenuEntry(
			  LOCTEXT("OpenGenericInstanceMenuOption", "Open Generic Object Mixer Instance"),
			  LOCTEXT("OpenGenericInstanceMenuOptionTooltip", "Open a generic object mixer instance that can take in a user-specified filter class."),
			  FSlateIcon(),
			  FUIAction(FExecuteAction::CreateLambda([]()
			  {
				  FGlobalTabmanager::Get()->TryInvokeTab(FObjectMixerEditorModule::Get().GetTabSpawnerId());
			  })));
		}

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ClearSoloStatesMenuOption","Clear Solo States"), 
			LOCTEXT("ClearSoloStatesMenuOptionTooltip","Remove the solo state from all rows in this list."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda(
				[this]()
				{
					if (const TSharedPtr<FObjectMixerEditorList> ListModel = GetListModelPtr().Pin())
					{
						ListModel->ClearSoloRows();
						ListModel->EvaluateAndSetEditorVisibilityPerRow();
					}
				})
			)
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("ProjectSettings", LOCTEXT("ShowProjectSettingsHeading", "Project Settings"));
	{
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowOptions = false;
		FPropertyEditorModule & PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		const TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObject(GetMutableDefault<UObjectMixerEditorSettings>());
		MenuBuilder.AddWidget(DetailsView, FText::GetEmpty(), true, false);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("WorldPicker", LOCTEXT("ShowWorldHeading", "World"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("ChooseWorldSubMenu", "Choose World"),
			LOCTEXT("ChooseWorldSubMenuToolTip", "Choose the world to display in the outliner."),
			FNewMenuDelegate::CreateRaw(this, &FActorMode::BuildWorldPickerMenu)
		);
	}
	MenuBuilder.EndSection();
}

TSharedRef<FSceneOutlinerFilter> FObjectMixerOutlinerMode::CreateShowOnlySelectedActorsFilter()
{
	auto IsActorSelected = [](const AActor* InActor)
	{
		return InActor && InActor->IsSelected();
	};
	return MakeShareable(new FActorFilter(FActorTreeItem::FFilterPredicate::CreateStatic(IsActorSelected), FSceneOutlinerFilter::EDefaultBehaviour::Fail, FActorTreeItem::FFilterPredicate::CreateStatic(IsActorSelected)));
}

TSharedRef<FSceneOutlinerFilter> FObjectMixerOutlinerMode::CreateHideTemporaryActorsFilter()
{
	return MakeShareable(new FActorFilter(FActorTreeItem::FFilterPredicate::CreateStatic([](const AActor* InActor)
		{
			return ((InActor->GetWorld() && InActor->GetWorld()->WorldType != EWorldType::PIE) || GEditor->ObjectsThatExistInEditorWorld.Get(InActor)) && !InActor->HasAnyFlags(EObjectFlags::RF_Transient);
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FObjectMixerOutlinerMode::CreateIsInCurrentLevelFilter()
{
	return MakeShareable(new FActorFilter(FActorTreeItem::FFilterPredicate::CreateStatic([](const AActor* InActor)
		{
			if (InActor->GetWorld())
			{
				return InActor->GetLevel() == InActor->GetWorld()->GetCurrentLevel();
			}

			return false;
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FObjectMixerOutlinerMode::CreateIsInCurrentDataLayersFilter()
{
	return MakeShareable(new FActorFilter(FActorTreeItem::FFilterPredicate::CreateStatic([](const AActor* InActor)
		{
			for (const UDataLayerInstance* DataLayerInstance : InActor->GetDataLayerInstances())
				{
					if (DataLayerInstance->IsInActorEditorContext())
					{
						return true;
					}
				}

				return false;
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FObjectMixerOutlinerMode::CreateIsInCurrentContentBundleFilter()
{
	return MakeShareable(new FActorFilter(FActorTreeItem::FFilterPredicate::CreateLambda([this](const AActor* InActor)
		{
			if (!WorldPartitionEditorModule || !WorldPartitionEditorModule->IsEditingContentBundle())
			{
				return true;
			}
			return InActor->GetContentBundleGuid().IsValid() && WorldPartitionEditorModule->IsEditingContentBundle(InActor->GetContentBundleGuid());
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FObjectMixerOutlinerMode::CreateHideComponentsFilter()
{
	return MakeShared<TSceneOutlinerPredicateFilter<FComponentTreeItem>>(TSceneOutlinerPredicateFilter<FComponentTreeItem>(
		FComponentTreeItem::FFilterPredicate::CreateStatic([](const UActorComponent*) { return false; }),
		FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FObjectMixerOutlinerMode::CreateHideLevelInstancesFilter()
{
	return MakeShareable(new FActorFilter(FActorTreeItem::FFilterPredicate::CreateStatic([](const AActor* Actor)
		{
			// Check if actor belongs to a LevelInstance
			if (const ULevelInstanceSubsystem* LevelInstanceSubsystem = Actor->GetWorld()->GetSubsystem<ULevelInstanceSubsystem>())
			{
				if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
				{
					if (!LevelInstanceSubsystem->IsEditingLevelInstance(ParentLevelInstance))
					{
						return false;
					}
				}
			}
			// Or if the actor itself is a LevelInstance editor instance
			return Cast<ALevelInstanceEditorInstanceActor>(Actor) == nullptr;
		}), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FObjectMixerOutlinerMode::CreateHideUnloadedActorsFilter()
{
	return MakeShareable(new FActorDescFilter(FActorDescTreeItem::FFilterPredicate::CreateStatic(
		[](const FWorldPartitionActorDescInstance* ActorDescInstance) { return false; }), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

TSharedRef<FSceneOutlinerFilter> FObjectMixerOutlinerMode::CreateHideEmptyFoldersFilter()
{
	return MakeShareable(new FFolderFilter(FFolderTreeItem::FFilterPredicate::CreateStatic(
		[](const FFolder& Folder) { return true; }), FSceneOutlinerFilter::EDefaultBehaviour::Pass));
}

static const FName DefaultContextBaseMenuName("SceneOutliner.DefaultContextMenuBase");
static const FName DefaultContextMenuName("SceneOutliner.DefaultContextMenu");

void FObjectMixerOutlinerMode::RegisterContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(DefaultContextBaseMenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(DefaultContextBaseMenuName);

		Menu->AddDynamicSection("DynamicHierarchySection", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			if(USceneOutlinerMenuContext* Context = InMenu->FindContext<USceneOutlinerMenuContext>())
			{
				if (TSharedPtr<SSceneOutliner> SharedOutliner = Context->SceneOutliner.Pin())
				{
					SSceneOutliner* SceneOutliner = SharedOutliner.Get();
					// NOTE: the name "Section" is used in many other places
					FToolMenuSection& Section = InMenu->FindOrAddSection("Section");
					Section.Label = LOCTEXT("HierarchySectionName", "Hierarchy");

					if (Context->bShowParentTree)
					{
						if (Context->NumSelectedItems == 0)
						{
							FSceneOutlinerMenuHelper::AddMenuEntryCreateFolder(Section, *SceneOutliner);
						}
						else
						{
							if (Context->NumSelectedItems == 1)
							{
								SceneOutliner->GetTree().GetSelectedItems()[0]->GenerateContextMenu(InMenu, *SceneOutliner);
							}

							// If we've only got folders selected, show the selection and edit sub menus
							if (Context->NumSelectedItems > 0 && Context->NumSelectedFolders == Context->NumSelectedItems)
							{
								Section.AddSubMenu(
									"SelectSubMenu",
									LOCTEXT("SelectSubmenu", "Select"),
									LOCTEXT("SelectSubmenu_Tooltip", "Select the contents of the current selection"),
									FNewToolMenuDelegate::CreateSP(SceneOutliner, &SSceneOutliner::FillSelectionSubMenu));
							}
						}
					}
				}
			}
		}));

		Menu->AddDynamicSection("DynamicMainSection", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			// We always create a section here, even if there is no parent so that clients can still extend the menu
			FToolMenuSection& Section = InMenu->AddSection("MainSection", LOCTEXT("OutlinerSectionName", "Outliner"));

			if (USceneOutlinerMenuContext* Context = InMenu->FindContext<USceneOutlinerMenuContext>())
			{
				if (TSharedPtr<SSceneOutliner> SharedOutliner = Context->SceneOutliner.Pin())
				{
					SSceneOutliner* SceneOutliner = SharedOutliner.Get();

					// Don't add any of these menu items if we're not showing the parent tree
					// Can't move worlds or level blueprints
					if (Context->bShowParentTree && Context->NumSelectedItems > 0 && Context->NumWorldsSelected == 0 && Context->SceneOutliner.IsValid())
					{
						Section.AddSubMenu(
							"MoveActorsTo",
							LOCTEXT("MoveActorsTo", "Move To"),
							LOCTEXT("MoveActorsTo_Tooltip", "Move selection to another folder"),
							FNewToolMenuDelegate::CreateSP(Context->SceneOutliner.Pin().Get(), &SSceneOutliner::FillFoldersSubMenu));
					}

					if (Context->bShowParentTree && Context->NumSelectedItems > 0 && Context->SceneOutliner.IsValid())
					{
						// Only add the menu option to wp levels
						if (!Context->bRepresentingGameWorld && Context->bRepresentingPartitionedWorld)
						{
							Section.AddMenuEntry(
								"ForceLoadItems",
								LOCTEXT("ForceLoad", "Force Load"),
								LOCTEXT("ForceLoadTooltip", "Keep the selected items loaded in the editor even when they don't overlap a loaded World Partition region"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(SceneOutliner, &SSceneOutliner::PinSelectedItems),
									FCanExecuteAction::CreateLambda([SceneOutliner, Context]()
									{
										if(Context->NumPinnedItems != Context->NumSelectedItems || Context->NumSelectedFolders > 0)
										{
											return SceneOutliner->CanPinSelectedItems();
										}
										return false;
									})));

							Section.AddMenuEntry(
								"ReleaseForceLoadItems",
								LOCTEXT("ReleaseForceLoad", "Release Force Load"),
								LOCTEXT("ReleaseForceLoadTooltip", "Allow the World Partition system to load and unload the selected items automatically"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(SceneOutliner, &SSceneOutliner::UnpinSelectedItems),
									FCanExecuteAction::CreateLambda([SceneOutliner, Context]()
									{
										if (Context->NumPinnedItems != 0 || Context->NumSelectedFolders > 0)
										{
											return SceneOutliner->CanUnpinSelectedItems();
										}
										return false;
									})));
							
						}
					}

					if (Context->NumSelectedItems > 0)
					{
						SceneOutliner->AddSourceControlMenuOptions(InMenu);
					}
				}
			}
		}));

		Menu->AddDynamicSection("DynamicActorEditorContext", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			USceneOutlinerMenuContext* Context = InMenu->FindContext<USceneOutlinerMenuContext>();
			if (Context && Context->bShowParentTree)
			{
				if (TSharedPtr<SSceneOutliner> SharedOutliner = Context->SceneOutliner.Pin())
				{
					FToolMenuSection& Section = InMenu->AddSection("ActorEditorContextSection", LOCTEXT("ActorEditorContextSectionName", "Actor Editor Context"));
					SSceneOutliner* SceneOutliner = SharedOutliner.Get();

					if ((Context->NumSelectedItems == 1) && (Context->NumSelectedFolders == 1))
					{
						SceneOutliner->GetTree().GetSelectedItems()[0]->GenerateContextMenu(InMenu, *SceneOutliner);

						Section.AddMenuEntry(
							"MakeCurrentFolder",
							LOCTEXT("MakeCurrentFolder", "Make Current Folder"),
							FText(),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([SceneOutliner]()
								{
									const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
									if (Selection.SelectedItems.Num() == 1)
									{
										FSceneOutlinerTreeItemPtr Item = Selection.SelectedItems[0].Pin();
										if (FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>())
										{
											if (FolderItem->World.IsValid())
											{
												const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MakeCurrentActorFolder", "Make Current Actor Folder"));
												FActorFolders::Get().SetActorEditorContextFolder(*FolderItem->World, FolderItem->GetFolder());
											}
										}
									}
								}),
								FCanExecuteAction::CreateLambda([SceneOutliner]
								{
									const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
									if (Selection.SelectedItems.Num() == 1)
									{
										FSceneOutlinerTreeItemPtr Item = Selection.SelectedItems[0].Pin();
										if (FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>())
										{
											return FolderItem->World.IsValid() &&
												(FolderItem->World->GetCurrentLevel() == FolderItem->GetFolder().GetRootObjectAssociatedLevel()) &&
												(FActorFolders::Get().GetActorEditorContextFolder(*FolderItem->World) != FolderItem->GetFolder());
										}
									}
									return false;
								})
							)
						);
					}

					const FObjectMixerOutlinerMode* Mode = static_cast<const FObjectMixerOutlinerMode*>(SceneOutliner->GetMode());
					check(Mode);

					Section.AddMenuEntry(
						"ClearCurrentFolder",
						LOCTEXT("ClearCurrentFolder", "Clear Current Folder"),
						FText(),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([Mode]()
							{
								if (Mode->RepresentingWorld.IsValid())
								{
									const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClearCurrentActorFolder", "Clear Current Actor Folder"));
									FActorFolders::Get().SetActorEditorContextFolder(*Mode->RepresentingWorld.Get(), FFolder::GetWorldRootFolder(Mode->RepresentingWorld.Get()));
								}
							}),
							FCanExecuteAction::CreateLambda([Mode]
							{
								return Mode->RepresentingWorld.IsValid() && !FActorFolders::Get().GetActorEditorContextFolder(*Mode->RepresentingWorld.Get()).IsNone();
							})
						)
					);
				}
			}
		}));
	}

	if (!ToolMenus->IsMenuRegistered(DefaultContextMenuName))
	{
		ToolMenus->RegisterMenu(DefaultContextMenuName, DefaultContextBaseMenuName);
	}
}

TSharedPtr<SWidget> FObjectMixerOutlinerMode::BuildContextMenu()
{
	const bool bTempSyncSelectionValue = GetMutableDefault<UObjectMixerEditorSettings>()->bSyncSelection;
	GetMutableDefault<UObjectMixerEditorSettings>()->bSyncSelection = true;
	SynchronizeSelection();
	GetMutableDefault<UObjectMixerEditorSettings>()->bSyncSelection = bTempSyncSelectionValue;
	
	RegisterContextMenu();

	const FSceneOutlinerItemSelection ItemSelection(SceneOutliner->GetSelection());

	USceneOutlinerMenuContext* ContextObject = NewObject<USceneOutlinerMenuContext>();
	ContextObject->SceneOutliner = StaticCastSharedRef<SSceneOutliner>(SceneOutliner->AsShared());
	ContextObject->bShowParentTree = SceneOutliner->GetSharedData().bShowParentTree;
	ContextObject->NumSelectedItems = ItemSelection.Num();
	ContextObject->NumSelectedFolders = ItemSelection.Num<FFolderTreeItem>();
	ContextObject->NumWorldsSelected = ItemSelection.Num<FWorldTreeItem>();
	ContextObject->bRepresentingGameWorld = bRepresentingWorldGameWorld;
	ContextObject->bRepresentingPartitionedWorld = bRepresentingWorldPartitionedWorld;

	int32 NumPinnedItems = 0;
	if (const UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition())
	{
		ItemSelection.ForEachItem<IActorBaseTreeItem>([WorldPartition, &NumPinnedItems](const IActorBaseTreeItem& ActorItem)
		{
			if (WorldPartition->IsActorPinned(ActorItem.GetGuid()))
			{
				++NumPinnedItems;
			}
			return true;
		});
	}
	ContextObject->NumPinnedItems = NumPinnedItems;

	FToolMenuContext Context(ContextObject);

	UObjectMixerEditorListMenuContext* ObjectMixerContextObject = NewObject<UObjectMixerEditorListMenuContext>();
	ObjectMixerContextObject->Data = {ListModelPtr.Pin()->GetSelectedTreeViewItems(), ListModelPtr.Pin()};

	Context.AddObject(ObjectMixerContextObject, [](UObject* InContext)
	{
		UObjectMixerEditorListMenuContext* CastContext = CastChecked<UObjectMixerEditorListMenuContext>(InContext);
		CastContext->Data.SelectedItems.Empty();
		CastContext->Data.ListModelPtr.Reset();
	});

	UObjectMixerEditorListMenuContext::RegisterObjectMixerDynamicCollectionsContextMenuExtension("LevelEditor.ActorContextMenu");
	UObjectMixerEditorListMenuContext::RegisterObjectMixerDynamicCollectionsContextMenuExtension("LevelEditor.ComponentContextMenu");
	UObjectMixerEditorListMenuContext::RegisterObjectMixerDynamicCollectionsContextMenuExtension("LevelEditor.ElementContextMenu");

	if (const UObjectMixerObjectFilter* Filter = GetListModelPtr().Pin()->GetMainObjectFilterInstance())
	{
		Filter->OnContextMenuContextCreated(Context);
	}

	const FName MenuName = "LevelEditor.LevelEditorSceneOutliner.ContextMenu";
	
	const FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	const TSharedPtr<ILevelEditor> LevelEditorPtr = LevelEditorModule.GetLevelEditorInstance().Pin();
	
	if (LevelEditorPtr.IsValid())
	{
		FLevelEditorContextMenu::InitMenuContext(Context, LevelEditorPtr, ELevelEditorMenuContext::SceneOutliner);
	}

	// Build up the menu for a selection
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

	for (const FToolMenuSection& Section : Menu->Sections)
	{
		if (Section.Blocks.Num() > 0)
		{
			return ToolMenus->GenerateWidget(Menu);
		}
	}

	return nullptr;
}

TSharedPtr<SWidget> FObjectMixerOutlinerMode::CreateContextMenu()
{
	constexpr bool bShouldSelect = false;
	constexpr bool bSelectEvenIfHidden = true;

	// Make sure that no components are selected
	TArray<UActorComponent*> SelectedComponents = GetSelectedComponentsInEditor();
	if (!SelectedComponents.IsEmpty())
	{
		// We want to be able to undo to regain the previous component selection
		const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClickingOnActorsContextMenu", "Clicking on Actors (context menu)"));
		SelectComponentsInEditor(SelectedComponents, bShouldSelect, bSelectEvenIfHidden);

		GUnrealEd->UpdatePivotLocationForSelection();
		GEditor->RedrawLevelEditingViewports(false);
	}

	return BuildContextMenu();
}

void FObjectMixerOutlinerMode::OnItemAdded(FSceneOutlinerTreeItemPtr Item)
{
	if (const FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
	{
		if (!Item->Flags.bIsFilteredOut)
		{
			++FilteredActorCount;

			// Synchronize selection with editor if enabled
			if (ShouldSyncSelectionFromEditor() && GetSelectedActorsInEditor().Contains(ActorItem->Actor.Get()))
			{
				if (const FObjectMixerEditorListRowActor* ObjectMixerItem = Item->CastTo<FObjectMixerEditorListRowActor>())
				{
					if (ObjectMixerItem->OverrideParent)
					{
						// If there's an OverrideParent, this actor may appear multiple times. We disambiguate the selection
						// based on a GUID that combines the parent and actor itself, but the general editor doesn't. To avoid
						// selecting the wrong one, skip selecting this new item if it wasn't previously selected.
						if (!SelectedIDs.Contains(ObjectMixerItem->GetID()))
						{
							return;
						}
					}
				}

				SceneOutliner->SetItemSelection(Item, true);
			}
		}
	}
	else if (Item->IsA<FActorDescTreeItem>())
	{
		if (!Item->Flags.bIsFilteredOut)
		{
			++FilteredActorCount;
			++FilteredUnloadedActorCount;
		}
	}
}

void FObjectMixerOutlinerMode::OnItemRemoved(FSceneOutlinerTreeItemPtr Item)
{
	if (Item->IsA<FActorTreeItem>())
	{
		if (!Item->Flags.bIsFilteredOut)
		{
			--FilteredActorCount;
		}
	}
	else if (Item->IsA<FActorDescTreeItem>())
	{
		if (!Item->Flags.bIsFilteredOut)
		{
			--FilteredActorCount;
			--FilteredUnloadedActorCount;
		}
	}
}

void FObjectMixerOutlinerMode::OnComponentsUpdated()
{
	SceneOutliner->FullRefresh();
}

void FObjectMixerOutlinerMode::OnLevelActorDeleted(AActor* Actor)
{
	ApplicableActors.Remove(Actor);
}

void FObjectMixerOutlinerMode::OnSelectUnloadedActors(const TArray<FGuid>& ActorGuids)
{
	if (UWorldPartition* WorldPartition = RepresentingWorld->GetWorldPartition())
	{
		TArray<FSceneOutlinerTreeItemPtr> ItemsToSelect;
		ItemsToSelect.Reserve(ActorGuids.Num());
		for (const FGuid& ActorGuid : ActorGuids)
		{
			if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid))
			{
				if (FSceneOutlinerTreeItemPtr ItemPtr = SceneOutliner->GetTreeItem(FActorDescTreeItem::ComputeTreeItemID(ActorDescInstance->GetGuid(), ActorDescInstance->GetContainerInstance())))
				{
					ItemsToSelect.Add(ItemPtr);
				}
			}
		}

		if (ItemsToSelect.Num())
		{
			SceneOutliner->SetItemSelection(ItemsToSelect, true);
			SceneOutliner->ScrollItemIntoView(ItemsToSelect.Last());

			if (const FActorDescTreeItem* ActorDescItem = ItemsToSelect.Last()->CastTo<FActorDescTreeItem>())
			{
				ActorDescItem->FocusActorBounds();
			}
		}
	}
}

void FObjectMixerOutlinerMode::OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* InActorDescInstance)
{
	ApplicableUnloadedActors.Remove(InActorDescInstance);
}

void FObjectMixerOutlinerMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr TreeItem, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	if (ShouldSyncSelectionToEditor())
	{
		SynchronizeAllSelectionsToEditor();
	}

	SelectedIDs.Empty();

	Algo::TransformIf(
		SceneOutliner->GetSelection().SelectedItems,
		SelectedIDs,
		[](const TWeakPtr<ISceneOutlinerTreeItem>& Item) -> bool { return Item.IsValid(); },
		[](const TWeakPtr<ISceneOutlinerTreeItem>& Item) -> FSceneOutlinerTreeItemID { return Item.Pin()->GetID(); }
	);
	
	bShouldTemporarilyForceSelectionSyncToEditor = false;
}

void FObjectMixerOutlinerMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item)
{
	// Get Actor from Actor or Component Row
	TWeakObjectPtr<AActor> Actor = nullptr;
	constexpr ObjectMixerOutliner::FWeakActorSelectorAcceptingComponents Selector;
	Selector(Item, Actor);
	
	if (Actor.IsValid())
	{
		ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor.Get());
		if (LevelInstance && FSlateApplication::Get().GetModifierKeys().IsAltDown())
		{
			if (LevelInstance->CanEnterEdit())
			{
				LevelInstance->EnterEdit();
			}
			else if (LevelInstance->CanExitEdit())
			{
				LevelInstance->ExitEdit();
			}
		}
		else if (Item->CanInteract())
		{
			FSceneOutlinerItemSelection Selection(SceneOutliner->GetSelection());
			if (Selection.Has<FActorTreeItem>())
			{
				const bool bActiveViewportOnly = false;
				GEditor->MoveViewportCamerasToActor(Selection.GetData<AActor*>(ObjectMixerOutliner::FActorSelector(this)), bActiveViewportOnly);
			}
		}
		else
		{
			const bool bActiveViewportOnly = false;
			GEditor->MoveViewportCamerasToActor(*Actor.Get(), bActiveViewportOnly);
		}
	}
	else if (const FActorDescTreeItem* ActorDescItem = Item->CastTo<FActorDescTreeItem>())
	{
		ActorDescItem->FocusActorBounds();
	}
}

void FObjectMixerOutlinerMode::OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType)
{
	const TArray<AActor*> Actors = Selection.GetData<AActor*>(ObjectMixerOutliner::FActorSelector(this));

	constexpr bool bShouldSelect = true;
	constexpr bool bSelectEvenIfHidden = false;
	if (ShouldSyncSelectionToEditor())
	{
		SelectActorsInEditor(Actors, bShouldSelect, bSelectEvenIfHidden);
	}
	else
	{
		SelectActorsInMixer(Actors, bShouldSelect, bSelectEvenIfHidden);
	}

	// Set keyboard focus to the SceneOutliner, so the user can perform keyboard commands that interact
	// with selected actors (such as Delete, to delete selected actors.)
	SceneOutliner->SetKeyboardFocus();

	SetAsMostRecentOutliner();
}

void FObjectMixerOutlinerMode::OnItemPassesFilters(const ISceneOutlinerTreeItem& Item)
{
	if (const FActorTreeItem* const ActorItem = Item.CastTo<FActorTreeItem>())
	{
		ApplicableActors.Add(ActorItem->Actor);
	}
	else if (const FActorDescTreeItem* const ActorDescItem = Item.CastTo<FActorDescTreeItem>(); ActorDescItem && ActorDescItem->IsValid())
	{
		ApplicableUnloadedActors.Add(ActorDescItem->ActorDescHandle);
	}
}

FReply FObjectMixerOutlinerMode::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();

	// Rename key: Rename selected actors (not rebindable, because it doesn't make much sense to bind.)
	if (InKeyEvent.GetKey() == EKeys::F2)
	{
		if (Selection.Num() == 1)
		{
			FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();

			if (ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract())
			{
				SceneOutliner->SetPendingRenameItem(ItemToRename);
				SceneOutliner->ScrollItemIntoView(ItemToRename);
			}

			return FReply::Handled();
		}
	}

	// F5 forces a full refresh
	else if (InKeyEvent.GetKey() == EKeys::F5)
	{
		SceneOutliner->FullRefresh();
		return FReply::Handled();
	}

	// Delete key: Delete selected actors (not rebindable, because it doesn't make much sense to bind.)
	// Use Delete and Backspace instead of Platform_Delete because the LevelEditor default Edit Delete is bound to both
	else if (InKeyEvent.GetKey() == EKeys::Delete || InKeyEvent.GetKey() == EKeys::BackSpace)
	{
		if (SceneOutliner->GetSharedData().CustomDelete.IsBound())
		{
			SceneOutliner->GetSharedData().CustomDelete.Execute(Selection.SelectedItems);
		}
		else
		{
			if (RepresentingWorld.IsValid())
			{
				GUnrealEd->Exec(RepresentingWorld.Get(), TEXT("DELETE"));
			}
		}
		return FReply::Handled();

	}

	/* Allow the user to scroll to the current selection (and expand if needed) by pressing the key bound to
	 * FEditorViewportCommands::Get().FocusViewportToSelection (Default: 'F')
	 */
	else
	{
		const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
		const FInputChord CheckChord( InKeyEvent.GetKey(), EModifierKey::FromBools(ModifierKeys.IsControlDown(), ModifierKeys.IsAltDown(), ModifierKeys.IsShiftDown(), ModifierKeys.IsCommandDown()) );

		// Use the keyboard shortcut bound to 'Focus Viewport To Selection'
		if (FEditorViewportCommands::Get().FocusViewportToSelection->HasActiveChord(CheckChord))
		{
			if (Selection.Num() == 1)
			{
				FSceneOutlinerTreeItemPtr ItemToFocus = Selection.SelectedItems[0].Pin();

				if (ItemToFocus.IsValid())
				{
					SceneOutliner->ScrollItemIntoView(ItemToFocus);
				}

				// Return Unhandled here so that the level editor viewport can handle this event and focus the selected item
				return FReply::Unhandled();
			}
		}
	}

	return FReply::Unhandled();
}

bool FObjectMixerOutlinerMode::CanDelete() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders > 0 && NumberOfFolders == ItemSelection.Num());
}

bool FObjectMixerOutlinerMode::CanRename() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders == 1 && NumberOfFolders == ItemSelection.Num());
}

bool FObjectMixerOutlinerMode::CanRenameItem(const ISceneOutlinerTreeItem& Item) const
{
	// Can only rename actor and folder items when in actor browsing mode
	return (Item.IsValid() && (Item.IsA<FActorTreeItem>() || Item.IsA<FFolderTreeItem>()));
}

bool FObjectMixerOutlinerMode::CanCut() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders > 0 && NumberOfFolders == ItemSelection.Num());
}

bool FObjectMixerOutlinerMode::CanCopy() const
{
	const FSceneOutlinerItemSelection ItemSelection = SceneOutliner->GetSelection();
	const uint32 NumberOfFolders = ItemSelection.Num<FFolderTreeItem>();
	return (NumberOfFolders > 0 && NumberOfFolders == ItemSelection.Num());
}

bool FObjectMixerOutlinerMode::CanPaste() const
{
	return CanPasteFoldersOnlyFromClipboard();
}

bool FObjectMixerOutlinerMode::HasErrors() const
{
	if (RepresentingWorld.IsValid() && !bRepresentingWorldGameWorld && bRepresentingWorldPartitionedWorld)
	{
		if (UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			bool bHasErrors = 0;

			WorldPartition->ForEachActorDescContainerInstance([&bHasErrors](UActorDescContainerInstance* ActorDescContainerInstance)
			{
				if (ActorDescContainerInstance->GetContainer()->HasInvalidActors())
				{
					bHasErrors = true;
				}
			});

			return bHasErrors;
		}
	}

	return false;
}

FText FObjectMixerOutlinerMode::GetErrorsText() const
{
	return LOCTEXT("WorldHasInvalidActorFiles", "The world contains invalid actor files. Click the Repair button to repair them.");
}

void FObjectMixerOutlinerMode::RepairErrors() const
{
	if (!bRepresentingWorldGameWorld && bRepresentingWorldPartitionedWorld)
	{
		if (UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition())
		{
			ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
			ISourceControlProvider& SourceControlProvider = SourceControlModule.GetProvider();

			TArray<FAssetData> InvalidActorAssets;
			WorldPartition->ForEachActorDescContainerInstance([&InvalidActorAssets](UActorDescContainerInstance* ActorDescContainerInstance)
			{
				for (const FAssetData& InvalidActor : ActorDescContainerInstance->GetContainer()->GetInvalidActors())
				{
					InvalidActorAssets.Add(InvalidActor);
				}
				ActorDescContainerInstance->GetContainer()->ClearInvalidActors();
			});

			TArray<FString> ActorFilesToDelete;
			TArray<FString> ActorFilesToRevert;
			{
				FScopedSlowTask SlowTask(InvalidActorAssets.Num(), LOCTEXT("UpdatingSourceControlStatus", "Updating source control status..."));
				SlowTask.MakeDialogDelayed(1.0f);

				for (const FAssetData& InvalidActorAsset : InvalidActorAssets)
				{
					FPackagePath PackagePath;
					if (FPackagePath::TryFromPackageName(InvalidActorAsset.PackageName, PackagePath))
					{
						const FString ActorFile = PackagePath.GetLocalFullPath();
						FSourceControlStatePtr SCState = SourceControlProvider.GetState(ActorFile, EStateCacheUsage::ForceUpdate);

						if (SCState.IsValid() && SCState->IsSourceControlled())
						{
							if (SCState->IsAdded())
							{
								ActorFilesToRevert.Add(ActorFile);
							}
							else
							{
								if (SCState->IsCheckedOut())
								{
									ActorFilesToRevert.Add(ActorFile);
								}

								ActorFilesToDelete.Add(ActorFile);
							}
						}
						else
						{
							IFileManager::Get().Delete(*ActorFile, false, true);
						}
					}

					SlowTask.EnterProgressFrame(1);
				}
			}

			if (ActorFilesToRevert.Num() || ActorFilesToDelete.Num())
			{
				FScopedSlowTask SlowTask(ActorFilesToRevert.Num() + ActorFilesToDelete.Num(), LOCTEXT("DeletingFiles", "Deleting files..."));
				SlowTask.MakeDialogDelayed(1.0f);

				if (ActorFilesToRevert.Num())
				{
					SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), ActorFilesToRevert);
					SlowTask.EnterProgressFrame(ActorFilesToRevert.Num());
				}
	
				if (ActorFilesToDelete.Num())
				{
					SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), ActorFilesToDelete);
					SlowTask.EnterProgressFrame(ActorFilesToDelete.Num());
				}
			}
		}
	}
}

bool FObjectMixerOutlinerMode::CanPasteFoldersOnlyFromClipboard() const
{
	// Intentionally not checking if the level is locked/hidden here, as it's better feedback for the user if they attempt to paste
	// and get the message explaining why it's failed, than just not having the option available to them.
	FString PasteString;
	FPlatformApplicationMisc::ClipboardPaste(PasteString);
	return PasteString.StartsWith("BEGIN FOLDERLIST");
}

bool FObjectMixerOutlinerMode::GetFolderNamesFromPayload(const FSceneOutlinerDragDropPayload& InPayload, TArray<FName>& OutFolders, FFolder::FRootObject& OutCommonRootObject) const
{
	return FFolder::GetFolderPathsAndCommonRootObject(InPayload.GetData<FFolder>(ObjectMixerOutliner::FFolderPathSelector()), OutFolders, OutCommonRootObject);
}

TSharedPtr<FDragDropOperation> FObjectMixerOutlinerMode::CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const
{
	FSceneOutlinerDragDropPayload DraggedObjects(InTreeItems);

	// If the drag contains only actors, we shortcut and create a simple FActorDragDropGraphEdOp rather than an FSceneOutlinerDragDrop composite op.
	if (DraggedObjects.Has<FActorTreeItem>() && !DraggedObjects.Has<FComponentTreeItem>() && !DraggedObjects.Has<FFolderTreeItem>())
	{
		return FActorDragDropGraphEdOp::New(DraggedObjects.GetData<TWeakObjectPtr<AActor>>(ObjectMixerOutliner::FWeakActorSelector()));
	}

	TSharedPtr<FSceneOutlinerDragDropOp> OutlinerOp = MakeShareable(new FSceneOutlinerDragDropOp());

	if (DraggedObjects.Has<FActorTreeItem>() || DraggedObjects.Has<FComponentTreeItem>())
	{
		TSharedPtr<FActorDragDropOp> ActorOperation = MakeShareable(new FActorDragDropGraphEdOp);
		ActorOperation->Init(DraggedObjects.GetData<TWeakObjectPtr<AActor>>(ObjectMixerOutliner::FWeakActorSelectorAcceptingComponents()));
		OutlinerOp->AddSubOp(ActorOperation);
	}

	if (DraggedObjects.Has<FFolderTreeItem>())
	{
		FFolder::FRootObject CommonRootObject;
		TArray<FName> DraggedFolders;
		if (GetFolderNamesFromPayload(DraggedObjects, DraggedFolders, CommonRootObject))
		{
			TSharedPtr<FFolderDragDropOp> FolderOperation = MakeShareable(new FFolderDragDropOp);
			FolderOperation->Init(DraggedFolders, RepresentingWorld.Get(), CommonRootObject);
			OutlinerOp->AddSubOp(FolderOperation);
		}
	}
	OutlinerOp->Construct();
	return OutlinerOp;
}

bool FObjectMixerOutlinerMode::ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const
{
	if (Operation.IsOfType<FSceneOutlinerDragDropOp>())
	{
		const auto& OutlinerOp = static_cast<const FSceneOutlinerDragDropOp&>(Operation);
		if (const auto& FolderOp = OutlinerOp.GetSubOp<FFolderDragDropOp>())
		{
			for (const auto& Folder : FolderOp->Folders)
			{
				OutPayload.DraggedItems.Add(SceneOutliner->GetTreeItem(FFolder(FolderOp->RootObject, Folder)));
			}
		}
		if (const auto& ActorOp = OutlinerOp.GetSubOp<FActorDragDropOp>())
		{
			for (const auto& Actor : ActorOp->Actors)
			{
				OutPayload.DraggedItems.Add(SceneOutliner->GetTreeItem(Actor.Get()));
			}
		}
		return true;
	}
	else if (Operation.IsOfType<FActorDragDropOp>())
	{
		for (const TWeakObjectPtr<AActor>& Actor : static_cast<const FActorDragDropOp&>(Operation).Actors)
		{
			OutPayload.DraggedItems.Add(SceneOutliner->GetTreeItem(Actor.Get()));
		}
		return true;
	}

	return false;
}

FFolder FObjectMixerOutlinerMode::GetWorldDefaultRootFolder() const
{
	return FFolder::GetWorldRootFolder(RepresentingWorld.Get());
}

void FObjectMixerOutlinerMode::SynchronizeComponentAndActorSelection()
{
	// Implementation based FActorMode::SynchronizeActorSelection, but we use GetSelectedActorsInEditor instead and handle components as well

	TArray<UActorComponent*> SelectedComponents = GetSelectedComponentsInEditor();

	// Deselect components in the tree that are no longer selected in the editor.
	const FSceneOutlinerItemSelection Selection(SceneOutliner->GetSelection());
	auto DeselectComponents = [this, &SelectedComponents](FComponentTreeItem& Item)
	{
		if (!Item.Component.IsValid() || !SelectedComponents.Contains(Item.Component.Get()))
		{
			SceneOutliner->SetItemSelection(Item.AsShared(), false);
			return;
		}
	};
	Selection.ForEachItem<FComponentTreeItem>(DeselectComponents);

	TArray<AActor*> SelectedActors = GetSelectedActorsInEditor();

	// Remove any actors from the selected list that own selected components, since they're necessarily selected when an owned component is
	// selected, but presumably not the user's editing target.
	// NOTE: this assumption needs to be revisited if we implement multi-select
	for (UActorComponent* Component : SelectedComponents)
	{
		if (Component)
		{
			SelectedActors.Remove(Component->GetOwner());
		}
	}
	
	// Deselect actors in the tree that are no longer selected in the editor (or owned by a selected component)
	auto DeselectActors = [this, &SelectedActors](FActorTreeItem& Item)
	{
		if (!Item.Actor.IsValid() || !SelectedActors.Contains(Item.Actor.Get()))
		{
			SceneOutliner->SetItemSelection(Item.AsShared(), false);
		}
	};
	Selection.ForEachItem<FActorTreeItem>(DeselectActors);

	// See if the tree view selector is pointing at a selected item
	bool bSelectorInSelectionSet = false;

	TArray<FSceneOutlinerTreeItemPtr> SelectedItems;
	for (UActorComponent* Component : SelectedComponents)
	{
		if (FSceneOutlinerTreeItemPtr ComponentItem = SceneOutliner->GetTreeItem(Component))
		{
			if (!bSelectorInSelectionSet && SceneOutliner->HasSelectorFocus(ComponentItem))
			{
				bSelectorInSelectionSet = true;
			}

			SelectedItems.Add(ComponentItem);
		}
	}

	Selection.ForEachItem<FObjectMixerEditorListRowUObject>([this, &SelectedActors](FObjectMixerEditorListRowUObject& Item)
	{
		if (UObject* Object = Item.ObjectSoftPtr.Get())
		{
			if (AActor* Actor = Object->GetTypedOuter<AActor>())
			{
				if (!SelectedActors.Contains(Actor))
				{
					// If there is a subobject currently selected whose parent is not in the selection being synced, deselect it
					SceneOutliner->SetItemSelection(Item.AsShared(), false);
				}
				else
				{
					// Otherwise, don't reselect the parent actor as one of its subobjects is already selected, to avoid multi-select issues
					SelectedActors.Remove(Actor);
				}
			}
		}
	});
	
	for (AActor* Actor : SelectedActors)
	{
		if (FSceneOutlinerTreeItemPtr ActorItem = SceneOutliner->GetTreeItem(Actor))
		{
			if (!bSelectorInSelectionSet && SceneOutliner->HasSelectorFocus(ActorItem))
			{
				bSelectorInSelectionSet = true;
			}

			SelectedItems.Add(ActorItem);
		}
	}

	// If NOT bSelectorInSelectionSet then we want to just move the selector to the first selected item.
	ESelectInfo::Type SelectInfo = bSelectorInSelectionSet ? ESelectInfo::Direct : ESelectInfo::OnMouseClick;
	SceneOutliner->AddToSelection(SelectedItems, SelectInfo);

	FSceneOutlinerDelegates::Get().SelectionChanged.Broadcast();
}

void FObjectMixerOutlinerMode::SynchronizeSelectedActorDescs()
{
	if (UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(RepresentingWorld.Get()))
	{
		const FSceneOutlinerItemSelection Selection = SceneOutliner->GetSelection();
		TArray<FWorldPartitionHandle> SelectedActorHandles = Selection.GetData<FWorldPartitionHandle>(ObjectMixerOutliner::FActorHandleSelector());

		WorldPartitionSubsystem->SelectedActorHandles.Empty();
		for (const FWorldPartitionHandle& ActorHandle : SelectedActorHandles)
		{
			WorldPartitionSubsystem->SelectedActorHandles.Add(ActorHandle);
		}
	}
}

void FObjectMixerOutlinerMode::SynchronizeSubObjectSelection()
{
	TArray<UObject*> SelectedSubObjects = GetSelectedSubObjectsInEditor();

	if (SelectedSubObjects.IsEmpty())
	{
		return;
	}

	const FSceneOutlinerItemSelection Selection = SceneOutliner->GetSelection();

	// Deselect any object rows that are not in the current selection
	Selection.ForEachItem<FObjectMixerEditorListRowUObject>([this, &SelectedSubObjects](FObjectMixerEditorListRowUObject& Item)
	{
		if (!Item.ObjectSoftPtr.IsValid() || !SelectedSubObjects.Contains(Item.ObjectSoftPtr.Get()))
		{
			SceneOutliner->SetItemSelection(Item.AsShared(), false);
		}
	});

	// Deselect any parent actors of the selected sub-objects to alleviate any multi-select issues
	TArray<AActor*> OwningActors;
	Algo::Transform(SelectedSubObjects, OwningActors, [](const UObject* Object)
	{
		return Object->GetTypedOuter<AActor>();
	});
	
	Selection.ForEachItem<FActorTreeItem>([this, &OwningActors](FActorTreeItem& Item)
	{
		if (Item.Actor.IsValid() && OwningActors.Contains(Item.Actor.Get()))
		{
			SceneOutliner->SetItemSelection(Item.AsShared(), false);
		}
	});
	
	bool bSelectorInSelectionSet = false;

	TArray<FSceneOutlinerTreeItemPtr> SelectedItems;
	for (UObject* SubObject : SelectedSubObjects)
	{
		if (FSceneOutlinerTreeItemPtr ObjectItem = SceneOutliner->GetTreeItem(SubObject))
		{
			if (!bSelectorInSelectionSet && SceneOutliner->HasSelectorFocus(ObjectItem))
			{
				bSelectorInSelectionSet = true;
			}

			SelectedItems.Add(ObjectItem);
		}
	}

	// If NOT bSelectorInSelectionSet then we want to just move the selector to the first selected item.
	ESelectInfo::Type SelectInfo = bSelectorInSelectionSet ? ESelectInfo::Direct : ESelectInfo::OnMouseClick;
	SceneOutliner->AddToSelection(SelectedItems, SelectInfo);

	FSceneOutlinerDelegates::Get().SelectionChanged.Broadcast();
}

FSceneOutlinerDragValidationInfo FObjectMixerOutlinerMode::ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const
{
	if (const UObjectMixerObjectFilter* Filter = GetListModelPtr().Pin()->GetMainObjectFilterInstance())
	{
		if (Filter->HasCustomDropHandling(DropTarget))
		{
			return Filter->ValidateDrop(DropTarget, Payload);
		}
	}

	if (Payload.Has<FFolderTreeItem>())
	{
		FFolder::FRootObject TargetRootObject = DropTarget.GetRootObject();
		FFolder::FRootObject CommonPayloadFoldersRootObject;
		TArray<FName> PayloadFolders;
		const bool bHasCommonRootObject = GetFolderNamesFromPayload(Payload, PayloadFolders, CommonPayloadFoldersRootObject);
		if (!bHasCommonRootObject)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("CantMoveFoldersWithMultipleRoots", "Cannot move folders with multiple roots"));
		}
		else if (CommonPayloadFoldersRootObject != TargetRootObject)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("CantChangeFoldersRoot", "Cannot change folders root"));
		}
	}

	if (const FActorTreeItem* ActorItem = DropTarget.CastTo<FActorTreeItem>())
	{
		const AActor* ActorTarget = ActorItem->Actor.Get();
		if (!ActorTarget)
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
		}

		const ILevelInstanceInterface* LevelInstanceTarget = Cast<ILevelInstanceInterface>(ActorTarget);
		const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
		if (LevelInstanceTarget)
		{
			check(LevelInstanceSubsystem);
			if (!LevelInstanceTarget->IsEditing())
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_AttachToClosedLevelInstance", "Cannot attach to LevelInstance which is not being edited"));
			}
		}
		else
		{
			if (Payload.Has<FFolderTreeItem>())
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("FoldersOnActorError", "Cannot attach folders to actors"));
			}

			if (!Payload.Has<FActorTreeItem>())
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
			}
		}

		FText AttachErrorMsg;
		bool bCanAttach = true;
		bool bDraggedOntoAttachmentParent = true;
		const auto& DragActors = Payload.GetData<TWeakObjectPtr<AActor>>(ObjectMixerOutliner::FWeakActorSelector());
		for (const auto& DragActorPtr : DragActors)
		{
			AActor* DragActor = DragActorPtr.Get();
			if (DragActor)
			{
				if (bCanAttach)
				{
					if (LevelInstanceSubsystem)
					{
						// Either all actors must be in a LevelInstance or none of them
						if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(DragActor))
						{
							if (!ParentLevelInstance->IsEditing())
							{
								return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_RemoveEditingLevelInstance", "Cannot detach from a LevelInstance which is not being edited"));
							}
						}

						if (!LevelInstanceSubsystem->CanMoveActorToLevel(DragActor, &AttachErrorMsg))
						{
							bCanAttach = bDraggedOntoAttachmentParent = false;
							break;
						}
					}

					if (DragActor->IsChildActor())
					{
						AttachErrorMsg = FText::Format(LOCTEXT("Error_AttachChildActor", "Cannot move {0} as it is a child actor."), FText::FromString(DragActor->GetActorLabel()));
						bCanAttach = bDraggedOntoAttachmentParent = false;
						break;
					}
					if (!LevelInstanceTarget && !GEditor->CanParentActors(ActorTarget, DragActor, &AttachErrorMsg))
					{
						bCanAttach = false;
					}
				}

				if (DragActor->GetSceneOutlinerParent() != ActorTarget)
				{
					bDraggedOntoAttachmentParent = false;
				}
			}
		}

		const FText ActorLabel = FText::FromString(ActorTarget->GetActorLabel());
		if (bDraggedOntoAttachmentParent)
		{
			if (DragActors.Num() == 1)
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleDetach, ActorLabel);
			}
			else
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleMultipleDetach, ActorLabel);
			}
		}
		else if (bCanAttach)
		{
			if (DragActors.Num() == 1)
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleAttach, ActorLabel);
			}
			else
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleMultipleAttach, ActorLabel);
			}
		}
		else
		{
			if (DragActors.Num() == 1)
			{
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, AttachErrorMsg);
			}
			else
			{
				const FText ReasonText = FText::Format(LOCTEXT("DropOntoText", "{0}. {1}"), ActorLabel, AttachErrorMsg);
				return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleMultipleAttach, ReasonText);
			}
		}
	}
	else if (DropTarget.IsA<FFolderTreeItem>() || DropTarget.IsA<FWorldTreeItem>() || DropTarget.IsA<FLevelTreeItem>())
	{
		const FFolderTreeItem* FolderItem = DropTarget.CastTo<FFolderTreeItem>();
		const FLevelTreeItem* LevelItem = DropTarget.CastTo<FLevelTreeItem>();
		// WorldTreeItem and LevelTreeItem are treated as root folders (path = none), with the difference that LevelTreeItem has a RootObject.
		const FFolder DestinationPath = FolderItem ? FolderItem->GetFolder() : (LevelItem ? FFolder(FFolder::GetOptionalFolderRootObject(LevelItem->Level.Get()).Get(FFolder::GetInvalidRootObject())) : GetWorldDefaultRootFolder());
		const FFolder::FRootObject& DestinationRootObject = DestinationPath.GetRootObject();
		ILevelInstanceInterface* LevelInstanceTarget = Cast<ILevelInstanceInterface>(DestinationPath.GetRootObjectPtr());
		if (LevelInstanceTarget && !LevelInstanceTarget->IsEditing())
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_DragInNonEditingLevelInstance", "Cannot drag into a LevelInstance which is not being edited"));
		}

		if (Payload.Has<FFolderTreeItem>())
		{
			FFolder::FRootObject CommonFolderRootObject;
			TArray<FName> DraggedFolders;
			if (GetFolderNamesFromPayload(Payload, DraggedFolders, CommonFolderRootObject))
			{
				// Iterate over all the folders that have been dragged
				for (const FName& DraggedFolder : DraggedFolders)
				{
					const FName Leaf = FEditorFolderUtils::GetLeafName(DraggedFolder);
					const FName Parent = FEditorFolderUtils::GetParentPath(DraggedFolder);

					if ((CommonFolderRootObject != DestinationRootObject) && FFolder::IsRootObjectValid(CommonFolderRootObject) && FFolder::IsRootObjectValid(DestinationRootObject))
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("SourceName"), FText::FromName(Leaf));
						FText Text = FText::Format(LOCTEXT("CantChangeFolderRoot", "Cannot change {SourceName} folder root"), Args);
						return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
					}

					if (Parent == DestinationPath.GetPath())
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("SourceName"), FText::FromName(Leaf));

						FText Text;
						if (DestinationPath.IsNone())
						{
							Text = FText::Format(LOCTEXT("FolderAlreadyAssignedRoot", "{SourceName} is already assigned to root"), Args);
						}
						else
						{
							Args.Add(TEXT("DestPath"), FText::FromName(DestinationPath.GetPath()));
							Text = FText::Format(LOCTEXT("FolderAlreadyAssigned", "{SourceName} is already assigned to {DestPath}"), Args);
						}

						return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
					}

					const FString DragFolderPath = DraggedFolder.ToString();
					const FString LeafName = Leaf.ToString();
					const FString DstFolderPath = DestinationPath.IsNone() ? FString() : DestinationPath.ToString();
					const FString NewPath = DstFolderPath / LeafName;

					if (FActorFolders::Get().ContainsFolder(*RepresentingWorld, FFolder(DestinationRootObject, FName(*NewPath))))
					{
						// The folder already exists
						FFormatNamedArguments Args;
						Args.Add(TEXT("DragName"), FText::FromString(LeafName));
						return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric,
							FText::Format(LOCTEXT("FolderAlreadyExistsRoot", "A folder called \"{DragName}\" already exists at this level"), Args));
					}
					else if (DragFolderPath == DstFolderPath || DstFolderPath.StartsWith(DragFolderPath + "/"))
					{
						// Cannot drag as a child of itself
						FFormatNamedArguments Args;
						Args.Add(TEXT("FolderPath"), FText::FromName(DraggedFolder));
						return FSceneOutlinerDragValidationInfo(
							ESceneOutlinerDropCompatibility::IncompatibleGeneric,
							FText::Format(LOCTEXT("ChildOfItself", "Cannot move \"{FolderPath}\" to be a child of itself"), Args));
					}
				}
			}
		}

		if (Payload.Has<FActorTreeItem>())
		{
			const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
			// Iterate over all the actors that have been dragged
			for (const TWeakObjectPtr<AActor>& WeakActor : Payload.GetData<TWeakObjectPtr<AActor>>(ObjectMixerOutliner::FWeakActorSelector()))
			{
				const AActor* Actor = WeakActor.Get();

				bool bActorContainedInLevelInstance = false;
				if (LevelInstanceSubsystem)
				{
					if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
					{
						if (!ParentLevelInstance->IsEditing())
						{
							return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("Error_RemoveEditingLevelInstance", "Cannot detach from a LevelInstance which is not being edited"));
						}
						bActorContainedInLevelInstance = true;
					}

					if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Actor))
					{
						FText Reason;
						if (!LevelInstanceSubsystem->CanMoveActorToLevel(Actor, &Reason))
						{
							return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Reason);
						}
					}
				}

				if (Actor->IsChildActor())
				{
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText::Format(LOCTEXT("Error_AttachChildActor", "Cannot move {0} as it is a child actor."), FText::FromString(Actor->GetActorLabel())));
				}
				else if ((Actor->GetFolderRootObject() != DestinationRootObject) && FFolder::IsRootObjectValid(Actor->GetFolderRootObject()) && FFolder::IsRootObjectValid(DestinationRootObject))
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("SourceName"), FText::FromString(Actor->GetActorLabel()));
					FText Text = FText::Format(LOCTEXT("CantChangeActorRoot", "Cannot change {SourceName} folder root"), Args);
					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
				}
				else if (Actor->GetFolder() == DestinationPath && !Actor->GetSceneOutlinerParent() && !bActorContainedInLevelInstance)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("SourceName"), FText::FromString(Actor->GetActorLabel()));

					FText Text;
					if (DestinationPath.IsNone())
					{
						Text = FText::Format(LOCTEXT("FolderAlreadyAssignedRoot", "{SourceName} is already assigned to root"), Args);
					}
					else
					{
						Args.Add(TEXT("DestPath"), FText::FromName(DestinationPath.GetPath()));
						Text = FText::Format(LOCTEXT("FolderAlreadyAssigned", "{SourceName} is already assigned to {DestPath}"), Args);
					}

					return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, Text);
				}
			}
		}

		// Everything else is a valid operation
		if (DestinationPath.IsNone())
		{
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, LOCTEXT("MoveToRoot", "Move to root"));
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("DestPath"), FText::FromName(DestinationPath.GetPath()));
			return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::CompatibleGeneric, FText::Format(LOCTEXT("MoveInto", "Move into \"{DestPath}\""), Args));
		}
	}
	else if (DropTarget.IsA<FComponentTreeItem>())
	{
		// we don't allow drag and drop on components for now
		return FSceneOutlinerDragValidationInfo(ESceneOutlinerDropCompatibility::IncompatibleGeneric, FText());
	}
	return FSceneOutlinerDragValidationInfo::Invalid();
}

void FObjectMixerOutlinerMode::OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const
{
	if (const UObjectMixerObjectFilter* Filter = GetListModelPtr().Pin()->GetMainObjectFilterInstance())
	{
		if (Filter->HasCustomDropHandling(DropTarget))
		{
			Filter->OnDrop(DropTarget, Payload, ValidationInfo);
			return;
		}
	}

	if (const FActorTreeItem* ActorItem = DropTarget.CastTo<FActorTreeItem>())
	{
		AActor* DropActor = ActorItem->Actor.Get();
		if (!DropActor)
		{
			return;
		}

		FMessageLog EditorErrors("EditorErrors");
		EditorErrors.NewPage(LOCTEXT("ActorAttachmentsPageLabel", "Actor attachment"));

		if (ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleMultipleDetach || ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleDetach)
		{
			const FScopedTransaction Transaction(LOCTEXT("UndoAction_DetachActors", "Detach actors"));

			TArray<TWeakObjectPtr<AActor>> DraggedActors = Payload.GetData<TWeakObjectPtr<AActor>>(ObjectMixerOutliner::FWeakActorSelector());
			for (const auto& WeakActor : DraggedActors)
			{
				if (auto* DragActor = WeakActor.Get())
				{
					// Detach from parent
					USceneComponent* RootComp = DragActor->GetRootComponent();
					if (RootComp && RootComp->GetAttachParent())
					{
						AActor* OldParent = RootComp->GetAttachParent()->GetOwner();
						// Attachment is persisted on the child so modify both actors for Undo/Redo but do not mark the Parent package dirty
						OldParent->Modify(/*bAlwaysMarkDirty=*/false);
						RootComp->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);

						DragActor->SetFolderPath_Recursively(OldParent->GetFolderPath());
					}
				}
			}
		}
		else if (ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleMultipleAttach || ValidationInfo.CompatibilityType == ESceneOutlinerDropCompatibility::CompatibleAttach)
		{
			// Show socket chooser if we have sockets to select

			if (ILevelInstanceInterface* TargetLevelInstance = Cast<ILevelInstanceInterface>(DropActor))
			{
				check(TargetLevelInstance->IsEditing());
				const FScopedTransaction Transaction(LOCTEXT("UndoAction_MoveActorsToLevelInstance", "Move actors to LevelInstance"));

				const FFolder DestinationPath = FFolder(FFolder::FRootObject(DropActor));
				auto MoveToDestination = [&DestinationPath](FFolderTreeItem& Item)
				{
					Item.MoveTo(DestinationPath);
				};
				Payload.ForEachItem<FFolderTreeItem>(MoveToDestination);

				// Since target root is directly the Level Instance, clear folder path
				TArray<AActor*> DraggedActors = Payload.GetData<AActor*>(ObjectMixerOutliner::FActorSelector(this));
				for (auto& Actor : DraggedActors)
				{
					Actor->SetFolderPath_Recursively(FName());
				}

				ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
				check(LevelInstanceSubsystem);
				LevelInstanceSubsystem->MoveActorsTo(TargetLevelInstance, DraggedActors);
			}
			else
			{
				auto PerformAttachment = [](FName SocketName, TWeakObjectPtr<AActor> Parent, const TArray<TWeakObjectPtr<AActor>> NewAttachments)
				{
					AActor* ParentActor = Parent.Get();
					if (ParentActor)
					{
						// modify parent and child
						const FScopedTransaction Transaction(LOCTEXT("UndoAction_PerformAttachment", "Attach actors"));

						// Attach each child
						for (auto& Child : NewAttachments)
						{
							AActor* ChildActor = Child.Get();
							if (GEditor->CanParentActors(ParentActor, ChildActor))
							{
								GEditor->ParentActors(ParentActor, ChildActor, SocketName);

								ChildActor->SetFolderPath_Recursively(ParentActor->GetFolderPath());
							}
						}
					}
				};

				TArray<TWeakObjectPtr<AActor>> DraggedActors = Payload.GetData<TWeakObjectPtr<AActor>>(ObjectMixerOutliner::FWeakActorSelector());
				//@TODO: Should create a menu for each component that contains sockets, or have some form of disambiguation within the menu (like a fully qualified path)
				// Instead, we currently only display the sockets on the root component
				USceneComponent* Component = DropActor->GetRootComponent();
				if ((Component != NULL) && (Component->HasAnySockets()))
				{
					// Create the popup
					UE_LOGF(LogObjectMixerEditor, Warning, "%s: Todo: SSocketChooserPopup", __FUNCTION__);
					// FSlateApplication::Get().PushMenu(
					// 	SceneOutliner->AsShared(),
					// 	FWidgetPath(),
					// 	SNew(SSocketChooserPopup)
					// 	.SceneComponent(Component)
					// 	.OnSocketChosen_Lambda(PerformAttachment, DropActor, MoveTemp(DraggedActors)),
					// 	FSlateApplication::Get().GetCursorPos(),
					// 	FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
					// );
				}
				else
				{
					PerformAttachment(NAME_None, DropActor, MoveTemp(DraggedActors));
				}
			}

		}
		// Report errors
		EditorErrors.Notify(NSLOCTEXT("ActorAttachmentError", "AttachmentsFailed", "Attachments Failed!"));
	}
	else if (DropTarget.IsA<FFolderTreeItem>() || DropTarget.IsA<FWorldTreeItem>() || DropTarget.IsA<FLevelTreeItem>())
	{
		const FFolderTreeItem* FolderItem = DropTarget.CastTo<FFolderTreeItem>();
		const FLevelTreeItem* LevelItem = DropTarget.CastTo<FLevelTreeItem>();
		// WorldTreeItem and LevelTreeItem are treated as root folders (path = none), with the difference that LevelTreeItem has a RootObject.
		const FFolder DestinationPath = FolderItem ? FolderItem->GetFolder() : (LevelItem ? FFolder(FFolder::GetOptionalFolderRootObject(LevelItem->Level.Get()).Get(FFolder::GetInvalidRootObject())) : GetWorldDefaultRootFolder());

		const FScopedTransaction Transaction(LOCTEXT("MoveOutlinerItems", "Move World Outliner Items"));

		auto MoveToDestination = [&DestinationPath](FFolderTreeItem& Item)
		{
			Item.MoveTo(DestinationPath);
		};
		Payload.ForEachItem<FFolderTreeItem>(MoveToDestination);

		// Set the folder path on all the dragged actors, and detach any that need to be moved
		if (Payload.Has<FActorTreeItem>())
		{
			TSet<const AActor*> ParentActors;
			TSet<const AActor*> ChildActors;

			TArray<AActor*> MovingActorsToValidRootObject;
			Payload.ForEachItem<FActorTreeItem>([&DestinationPath, &ParentActors, &ChildActors, &MovingActorsToValidRootObject](const FActorTreeItem& ActorItem)
			{
				AActor* Actor = ActorItem.Actor.Get();
				if (Actor)
				{
					// First mark this object as a parent, then set its children's path
					ParentActors.Add(Actor);

					const FFolder SrcFolder = Actor->GetFolder();

					// If the folder root object changes, 1st pass will put actors at root. 2nd pass will set the destination path.
					FName NewPath = (SrcFolder.GetRootObject() == DestinationPath.GetRootObject()) ? DestinationPath.GetPath() : NAME_None;

					Actor->SetFolderPath(NewPath);
					FActorEditorUtils::TraverseActorTree_ParentFirst(Actor, [&](AActor* InActor) {
						ChildActors.Add(InActor);
						InActor->SetFolderPath(NewPath);
						return true;
						}, false);

					if ((Actor->GetFolderRootObject() != DestinationPath.GetRootObject()) && SrcFolder.IsRootObjectPersistentLevel() && (DestinationPath.IsRootObjectValid() && !DestinationPath.IsRootObjectPersistentLevel()))
					{
						MovingActorsToValidRootObject.Add(Actor);
					}
				}
			});

			// Detach parent actors
			for (const AActor* Parent : ParentActors)
			{
				auto* RootComp = Parent->GetRootComponent();

				// We don't detach if it's a child of another that's been dragged
				if (RootComp && RootComp->GetAttachParent() && !ChildActors.Contains(Parent))
				{
					if (AActor* OldParentActor = RootComp->GetAttachParent()->GetOwner())
					{
						// Attachment is persisted on the child so modify both actors for Undo/Redo but do not mark the Parent package dirty
						OldParentActor->Modify(/*bAlwaysMarkDirty=*/false);
					}
					RootComp->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
				}
			}

			auto MoveActorsToLevel = [](const TArray<AActor*>& InActorsToMove, ULevel* InDestLevel, const FName& InDestinationPath)
			{
				// We are moving actors to another level
				const bool bWarnAboutReferences = true;
				const bool bWarnAboutRenaming = true;
				const bool bMoveAllOrFail = true;
				TArray<AActor*> MovedActors;
				if (!EditorLevelUtils::MoveActorsToLevel(InActorsToMove, InDestLevel, bWarnAboutReferences, bWarnAboutRenaming, bMoveAllOrFail, &MovedActors))
				{
					UE_LOGF(LogObjectMixerEditor, Warning, "Failed to move actors because not all actors could be moved");
				}
				// Once moved, update actors folder path
				for (AActor* Actor : MovedActors)
				{
					Actor->SetFolderPath_Recursively(InDestinationPath);
				}
			};

			if (DestinationPath.IsRootObjectPersistentLevel())
			{
				const ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
				check(LevelInstanceSubsystem);
				ULevel* DestinationLevel = RepresentingWorld->PersistentLevel;
				check(DestinationLevel);

				TArray<AActor*> LevelInstanceActorsToMove;
				TArray<AActor*> ActorsToMoveToPersistentLevel;
				Payload.ForEachItem<FActorTreeItem>([LevelInstanceSubsystem, &LevelInstanceActorsToMove, &ActorsToMoveToPersistentLevel](const FActorTreeItem& ActorItem)
				{
					AActor* Actor = ActorItem.Actor.Get();
					if (const ILevelInstanceInterface* ParentLevelInstance = LevelInstanceSubsystem->GetParentLevelInstance(Actor))
					{
						check(ParentLevelInstance->IsEditing());
						LevelInstanceActorsToMove.Add(Actor);
					}
					else
					{
						const FFolder ActorSrcFolder = Actor->GetFolder();
						if (ActorSrcFolder.IsRootObjectValid() && !ActorSrcFolder.IsRootObjectPersistentLevel())
						{
							ActorsToMoveToPersistentLevel.Add(Actor);
						}
					}
				});

				// We are moving actors outside of an editing level instance to a folder (or root) into the persistent level.
				if (LevelInstanceActorsToMove.Num() > 0)
				{
					TArray<AActor*> MovedActors;
					LevelInstanceSubsystem->MoveActorsToLevel(LevelInstanceActorsToMove, DestinationLevel, &MovedActors);
					// Once moved, update actors folder path
					for (AActor* Actor : MovedActors)
					{
						Actor->SetFolderPath_Recursively(DestinationPath.GetPath());
					}
				}
				if (ActorsToMoveToPersistentLevel.Num() > 0)
				{
					MoveActorsToLevel(ActorsToMoveToPersistentLevel, DestinationLevel, DestinationPath.GetPath());
				}
			}
			else if (MovingActorsToValidRootObject.Num())
			{
				if (ILevelInstanceInterface* TargetLevelInstance = Cast<ILevelInstanceInterface>(DestinationPath.GetRootObjectPtr()))
				{
					// We are moving actors inside an editing level instance
					check(TargetLevelInstance->IsEditing());

					ULevelInstanceSubsystem* LevelInstanceSubsystem = RepresentingWorld->GetSubsystem<ULevelInstanceSubsystem>();
					check(LevelInstanceSubsystem);
					TArray<AActor*> MovedActors;
					LevelInstanceSubsystem->MoveActorsTo(TargetLevelInstance, MovingActorsToValidRootObject, &MovedActors);
					// Once moved, update actors folder path
					for (AActor* Actor : MovedActors)
					{
						Actor->SetFolderPath_Recursively(DestinationPath.GetPath());
					}
				}
				else if (ULevel* DestinationLevel = Cast<ULevel>(DestinationPath.GetRootObjectPtr()))
				{
					MoveActorsToLevel(MovingActorsToValidRootObject, DestinationLevel, DestinationPath.GetPath());
				}
			}
		}
	}
}

FFolder FObjectMixerOutlinerMode::CreateNewFolder()
{
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));
	TArray<FFolder> SelectedFolders = SceneOutliner->GetSelection().GetData<FFolder>(ObjectMixerOutliner::FFolderPathSelector());
	const FFolder NewFolderName = FActorFolders::Get().GetDefaultFolderForSelection(*RepresentingWorld, &SelectedFolders);
	FActorFolders::Get().CreateFolderContainingSelection(*RepresentingWorld, NewFolderName);

	return NewFolderName;
}

FFolder FObjectMixerOutlinerMode::GetFolder(const FFolder& ParentPath, const FName& LeafName)
{
	// Return a unique folder under the provided parent path & root object and using the provided leaf name
	return FActorFolders::Get().GetFolderName(*RepresentingWorld, ParentPath, LeafName);
}

bool FObjectMixerOutlinerMode::CreateFolder(const FFolder& NewPath)
{
	return FActorFolders::Get().CreateFolder(*RepresentingWorld, NewPath);
}

bool FObjectMixerOutlinerMode::ReparentItemToFolder(const FFolder& FolderPath, const FSceneOutlinerTreeItemPtr& Item)
{
	if (FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
	{
		// Make sure actor has the same root object before updating path
		if (ActorItem->Actor->GetFolderRootObject() == FolderPath.GetRootObject())
		{
			ActorItem->Actor->SetFolderPath_Recursively(FolderPath.GetPath());
			return true;
		}
	}
	return false;
}

namespace ObjectMixerActorBrowsingModeUtils
{
	static void RecursiveFolderExpandChildren(SSceneOutliner* SceneOutliner, const FSceneOutlinerTreeItemPtr& Item)
	{
		if (Item.IsValid())
		{
			for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : Item->GetChildren())
			{
				FSceneOutlinerTreeItemPtr ChildPtr = Child.Pin();
				SceneOutliner->SetItemExpansion(ChildPtr, true);
				RecursiveFolderExpandChildren(SceneOutliner, ChildPtr);
			}
		}
	}

	// Get a list of unique actors to select as descendents of the Item in the SceneOutliner
	// Note that this also selects any folders encountered during the search
	static void RecursiveGetActorsToSelect(SSceneOutliner* SceneOutliner, const FSceneOutlinerTreeItemPtr& Item, bool bSelectImmediateChildrenOnly, TArray<AActor*>& OutActorsToSelect)
	{
		if (Item.IsValid())
		{
			// If the current item is an actor, ensure to select it as well
			if (FActorTreeItem* ActorItem = Item->CastTo<FActorTreeItem>())
			{
				if (AActor* Actor = ActorItem->Actor.Get())
				{
					OutActorsToSelect.AddUnique(Actor);
				}
			}
			// Select all children
			for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : Item->GetChildren())
			{
				FSceneOutlinerTreeItemPtr ChildPtr = Child.Pin();
				if (ChildPtr.IsValid())
				{
					if (FActorTreeItem* ActorItem = ChildPtr->CastTo<FActorTreeItem>())
					{
						if (AActor* Actor = ActorItem->Actor.Get())
						{
							OutActorsToSelect.AddUnique(Actor);
						}
					}
					else if (FFolderTreeItem* FolderItem = ChildPtr->CastTo<FFolderTreeItem>())
					{
						SceneOutliner->SetItemSelection(FolderItem->AsShared(), true);
					}

					if (!bSelectImmediateChildrenOnly)
					{
						for (const TWeakPtr<ISceneOutlinerTreeItem>& Grandchild : ChildPtr->GetChildren())
						{
							RecursiveGetActorsToSelect(SceneOutliner, Grandchild.Pin(), bSelectImmediateChildrenOnly, OutActorsToSelect);
						}
					}
				}
			}
		}
	}

	static void RecursiveAddItemsToActorGuidList(const TArray<FSceneOutlinerTreeItemPtr>& Items, TArray<FGuid>& List, bool bSearchForHiddenUnloadedActors)
	{
		// In the case where we want the list of unloaded actors under a folder and the option to hide unloaded actors is enabled, we need to find them through FActorFolders 
		TMap<UWorld*, TSet<FName>> UnloadedActorsFolderPaths;

		for (const FSceneOutlinerTreeItemPtr& Item : Items)
		{
			if (const FActorDescTreeItem* const ActorDescTreeItem = Item->CastTo<FActorDescTreeItem>())
			{
				List.Add(ActorDescTreeItem->GetGuid());
			}
			else if (const FActorTreeItem* const ActorTreeItem = Item->CastTo<FActorTreeItem>())
			{
				if (ActorTreeItem->Actor.IsValid())
				{
					List.Add(ActorTreeItem->Actor->GetActorGuid());
				}
			}
			else if (const FActorFolderTreeItem* const ActorFolderTreeItem = Item->CastTo<FActorFolderTreeItem>(); ActorFolderTreeItem && bSearchForHiddenUnloadedActors)
			{
				if (UWorld* FolderWorld = ActorFolderTreeItem->World.Get())
				{
					UnloadedActorsFolderPaths.FindOrAdd(FolderWorld).Add(ActorFolderTreeItem->GetPath());
				}
			}

			TArray<FSceneOutlinerTreeItemPtr> ChildrenItems;
			for (const auto& Child : Item->GetChildren())
			{
				if (Child.IsValid())
				{
					ChildrenItems.Add(Child.Pin());
				}
			}
						
			if (ChildrenItems.Num())
			{
				RecursiveAddItemsToActorGuidList(ChildrenItems, List, bSearchForHiddenUnloadedActors);
			}
		}

		
		for (const TTuple<UWorld*, TSet<FName>>& Pair : UnloadedActorsFolderPaths)
		{
			FActorFolders::ForEachActorDescInstanceInFolders(*Pair.Key, Pair.Value, [&List](const FWorldPartitionActorDescInstance* ActorDescInstance)
			{
				if (!ActorDescInstance->IsLoaded())
				{
					List.Add(ActorDescInstance->GetGuid());
				}
				return true;
			});
		}
	};

	static bool CanChangePinnedStates(const TArray<FSceneOutlinerTreeItemPtr>& InItems)
	{
		for (const FSceneOutlinerTreeItemPtr& Item : InItems)
		{
			if (Item->ShouldShowPinnedState())
			{
				return true;
			}
			else if (const FActorFolderTreeItem* FolderItem = Item->CastTo<FActorFolderTreeItem>(); FolderItem && FolderItem->CanChangeChildrenPinnedState())
			{
				return true;
			}
		}

		return false;
	}
}

void FObjectMixerOutlinerMode::SelectFoldersDescendants(const TArray<FFolderTreeItem*>& FolderItems, bool bSelectImmediateChildrenOnly)
{
	// Expand everything before beginning selection
	for (FFolderTreeItem* Folder : FolderItems)
	{
		FSceneOutlinerTreeItemPtr FolderPtr = Folder->AsShared();
		SceneOutliner->SetItemExpansion(FolderPtr, true);
		if (!bSelectImmediateChildrenOnly)
		{
			ObjectMixerActorBrowsingModeUtils::RecursiveFolderExpandChildren(SceneOutliner, FolderPtr);
		}
	}

	// batch selection
	TArray<AActor*> ActorsToSelect;
	for (FFolderTreeItem* Folder : FolderItems)
	{
		ObjectMixerActorBrowsingModeUtils::RecursiveGetActorsToSelect(SceneOutliner, Folder->AsShared(), bSelectImmediateChildrenOnly, ActorsToSelect);
	}

	constexpr bool bShouldSelect = true;
	constexpr bool bSelectEvenIfHidden = true;
	if (ShouldSyncSelectionToEditor())
	{
		SelectActorsInEditor(ActorsToSelect, bShouldSelect, bSelectEvenIfHidden);
	}
	else
	{
		SelectActorsInMixer(ActorsToSelect, bShouldSelect, bSelectEvenIfHidden);
	}
}

bool FObjectMixerOutlinerMode::CanPinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const
{
	return ObjectMixerActorBrowsingModeUtils::CanChangePinnedStates(InItems);
}

void FObjectMixerOutlinerMode::PinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems)
{
	UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition();
	if (!WorldPartition)
	{
		return;
	}

	TArray<FGuid> ActorsGuidsToPin;
	// If Unloaded actors are hidden and we are pinning folders we need to find them through FActorFolders
	const bool bSearchForHiddenUnloadedActors = bHideUnloadedActors;
	ObjectMixerActorBrowsingModeUtils::RecursiveAddItemsToActorGuidList(InItems, ActorsGuidsToPin, bSearchForHiddenUnloadedActors);

	if (ActorsGuidsToPin.Num())
	{
		WorldPartition->PinActors(ActorsGuidsToPin);

		TArray<AActor*> ActorsToPin;
		ActorsToPin.Reserve(ActorsGuidsToPin.Num());

		AActor* LastPinnedActor = nullptr;
		for (const FGuid& ActorGuid : ActorsGuidsToPin)
		{
			if (FWorldPartitionHandle ActorHandle(WorldPartition, ActorGuid); ActorHandle.IsValid())
			{
				if (AActor* PinnedActor = ActorHandle.GetActor())
				{
					ActorsToPin.Add(PinnedActor);
					LastPinnedActor = PinnedActor;
				}
			}
		}

		constexpr bool bShouldSelect = true;
		constexpr bool bSelectEvenIfHidden = true;
		SelectActorsInEditor(ActorsToPin, bShouldSelect, bSelectEvenIfHidden);

		if (LastPinnedActor)
		{
			SceneOutliner->OnItemAdded(LastPinnedActor, SceneOutliner::ENewItemAction::ScrollIntoView);
		}
	}
}

bool FObjectMixerOutlinerMode::CanUnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const
{
	return ObjectMixerActorBrowsingModeUtils::CanChangePinnedStates(InItems);
}

void FObjectMixerOutlinerMode::UnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems)
{
	UWorldPartition* const WorldPartition = RepresentingWorld->GetWorldPartition();
	if (!WorldPartition)
	{
		return;
	}

	TArray<FGuid> ActorGuidsToUnpin;
	// No need to search for hidden unloaded actors when unloading
	const bool bSearchForHiddenUnloadedActors = false;
	ObjectMixerActorBrowsingModeUtils::RecursiveAddItemsToActorGuidList(InItems, ActorGuidsToUnpin, bSearchForHiddenUnloadedActors);

	if (ActorGuidsToUnpin.Num())
	{
		TArray<AActor*> ActorsToUnpin;
		ActorsToUnpin.Reserve(ActorGuidsToUnpin.Num());

		for (const FGuid& ActorGuid : ActorGuidsToUnpin)
		{
			if (FWorldPartitionHandle ActorHandle(WorldPartition, ActorGuid); ActorHandle.IsValid())
			{
				if (AActor* PinnedActor = ActorHandle.GetActor())
				{
					ActorsToUnpin.Add(PinnedActor);
				}
			}
		}

		constexpr bool bShouldSelect = false;
		constexpr bool bSelectEvenIfHidden = true;
		SelectActorsInEditor(ActorsToUnpin, bShouldSelect, bSelectEvenIfHidden);

		WorldPartition->UnpinActors(ActorGuidsToUnpin);
	}
}

void FObjectMixerOutlinerMode::SynchronizeSelection()
{
	if (ShouldSyncSelectionFromEditor())
	{
		SynchronizeComponentAndActorSelection();
		SynchronizeSelectedActorDescs();
		SynchronizeSubObjectSelection();
	}
	else
	{
		SceneOutliner->ClearSelection();

		for (const FSceneOutlinerTreeItemID& TreeItemID : SelectedIDs)
		{
			const FSceneOutlinerTreeItemPtr Item = SceneOutliner->GetTreeItem(TreeItemID);
			if (Item.IsValid())
			{
				SceneOutliner->SetItemSelection(Item, true);
			}
		}
	}

	StaticCast<SObjectMixerEditorList*>(SceneOutliner)->GetOnSelectionSynchronized().Broadcast();
	
	bShouldTemporarilyForceSelectionSyncFromEditor = false;
}

FCreateSceneOutlinerMode FObjectMixerOutlinerMode::CreateFolderPickerMode(const FFolder::FRootObject& InRootObject) const
{
	auto MoveSelectionTo = [this, InRootObject](const FSceneOutlinerTreeItemRef& NewParent)
	{
		if (FWorldTreeItem* WorldItem = NewParent->CastTo<FWorldTreeItem>())
		{
			SceneOutliner->MoveSelectionTo(GetWorldDefaultRootFolder());
		}
		else if (FFolderTreeItem* FolderItem = NewParent->CastTo<FFolderTreeItem>())
		{
			SceneOutliner->MoveSelectionTo(FolderItem->GetFolder());
		}
		else if (FActorTreeItem* ActorItem = NewParent->CastTo<FActorTreeItem>())
		{
			if (FFolder::IsRootObjectValid(InRootObject))
			{
				SceneOutliner->MoveSelectionTo(FFolder(InRootObject));
			}
		}
		else if (FLevelTreeItem* LevelItem = NewParent->CastTo<FLevelTreeItem>())
		{
			if (FFolder::IsRootObjectValid(InRootObject))
			{
				SceneOutliner->MoveSelectionTo(FFolder(InRootObject));
			}
		}
	};

	return FCreateSceneOutlinerMode::CreateLambda([this, MoveSelectionTo, InRootObject](SSceneOutliner* Outliner)
	{
		return new FActorFolderPickingMode(Outliner, FOnSceneOutlinerItemPicked::CreateLambda(MoveSelectionTo), nullptr, InRootObject);
	});
}

void FObjectMixerOutlinerMode::OnDuplicateSelected()
{
	GUnrealEd->Exec(RepresentingWorld.Get(), TEXT("DUPLICATE"));
}

void FObjectMixerOutlinerMode::OnEditCutActorsBegin()
{
	// Only a callback in actor browsing mode
	SceneOutliner->CopyFoldersBegin();
	SceneOutliner->DeleteFoldersBegin();
}

void FObjectMixerOutlinerMode::OnEditCutActorsEnd()
{
	// Only a callback in actor browsing mode
	SceneOutliner->CopyFoldersEnd();
	SceneOutliner->DeleteFoldersEnd();
}

void FObjectMixerOutlinerMode::OnEditCopyActorsBegin()
{
	// Only a callback in actor browsing mode
	SceneOutliner->CopyFoldersBegin();
}

void FObjectMixerOutlinerMode::OnEditCopyActorsEnd()
{
	// Only a callback in actor browsing mode
	SceneOutliner->CopyFoldersEnd();
}

void FObjectMixerOutlinerMode::OnEditPasteActorsBegin()
{
	// Only a callback in actor browsing mode
	const TArray<FName> FolderPaths = SceneOutliner->GetClipboardPasteFolders();
	SceneOutliner->PasteFoldersBegin(FolderPaths);
}

void FObjectMixerOutlinerMode::OnEditPasteActorsEnd()
{
	// Only a callback in actor browsing mode
	SceneOutliner->PasteFoldersEnd();
}

void FObjectMixerOutlinerMode::OnDuplicateActorsBegin()
{
	// Only a callback in actor browsing mode
	FFolder::FRootObject CommonRootObject;
	TArray<FName> SelectedFolderPaths;
	FFolder::GetFolderPathsAndCommonRootObject(SceneOutliner->GetSelection().GetData<FFolder>(ObjectMixerOutliner::FFolderPathSelector()), SelectedFolderPaths, CommonRootObject);
	SceneOutliner->PasteFoldersBegin(SelectedFolderPaths);
}

void FObjectMixerOutlinerMode::OnDuplicateActorsEnd()
{
	// Only a callback in actor browsing mode
	SceneOutliner->PasteFoldersEnd();
}

void FObjectMixerOutlinerMode::OnDeleteActorsBegin()
{
	SceneOutliner->DeleteFoldersBegin();
}

void FObjectMixerOutlinerMode::OnDeleteActorsEnd()
{
	SceneOutliner->DeleteFoldersEnd();
}

bool FObjectMixerOutlinerMode::ShouldAllowHybridRows() const
{
	if (const TSharedPtr<FObjectMixerEditorList> PinnedList = GetListModelPtr().Pin())
	{
		if (const UObjectMixerObjectFilter* ObjectFilter = PinnedList->GetMainObjectFilterInstance())
		{
			return ObjectFilter->ShouldAllowHybridRows();
		}
	}

	return true;
}

FObjectMixerOutlinerModeConfig* FObjectMixerOutlinerMode::GetMutableConfig() const
{
	FName OutlinerIdentifier = SceneOutliner->GetOutlinerIdentifier();

	if (OutlinerIdentifier.IsNone())
	{
		return nullptr;
	}

	return &GetMutableDefault<UObjectMixerOutlinerModeEditorConfig>()->Browsers.FindOrAdd(OutlinerIdentifier);
}


const FObjectMixerOutlinerModeConfig* FObjectMixerOutlinerMode::GetConstConfig() const
{
	FName OutlinerIdentifier = SceneOutliner->GetOutlinerIdentifier();

	if (OutlinerIdentifier.IsNone())
	{
		return nullptr;
	}

	return GetDefault<UObjectMixerOutlinerModeEditorConfig>()->Browsers.Find(OutlinerIdentifier);
}

void FObjectMixerOutlinerMode::SaveConfig()
{
	GetMutableDefault<UObjectMixerOutlinerModeEditorConfig>()->SaveEditorConfig();
}

bool FObjectMixerOutlinerMode::CompareItemWithClassName(SceneOutliner::FilterBarType InItem, const TSet<FTopLevelAssetPath>& AssetClassPaths) const
{
	// Type filtering only supported for Actors (and unloaded actors) currently
	if (const FActorTreeItem* ActorItem = InItem.CastTo<FActorTreeItem>())
	{
		AActor* Actor = ActorItem->Actor.Get();

		if(!Actor)
		{
			return false;
		}

		FTopLevelAssetPath AssetClassPath = Actor->GetClass()->GetClassPathName();

		// For Blueprints, we check both the parent type (e.g Pawn/Actor etc) and the Blueprint class itself
		if (UBlueprint* ClassBP = UBlueprint::GetBlueprintFromClass(Actor->GetClass()))
		{
			return AssetClassPaths.Contains(ClassBP->GetClass()->GetClassPathName()) || AssetClassPaths.Contains(AssetClassPath);
		}
		
		return AssetClassPaths.Contains(AssetClassPath);
	}
	else if (const FActorDescTreeItem* ActorDescItem = InItem.CastTo<FActorDescTreeItem>())
	{
		if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
		{
			// For Unloaded Actors, grab the native class 
			FTopLevelAssetPath ClassPath = ActorDescInstance->GetNativeClass();
			return AssetClassPaths.Contains(ClassPath);
		}
	}

	return CompareItemWithClassName(InItem, AssetClassPaths);
}

#undef LOCTEXT_NAMESPACE
