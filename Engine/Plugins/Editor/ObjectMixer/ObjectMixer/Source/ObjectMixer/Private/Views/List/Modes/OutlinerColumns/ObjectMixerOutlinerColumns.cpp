// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/Modes/OutlinerColumns/ObjectMixerOutlinerColumns.h"

#include "ObjectMixerEditorStyle.h"
#include "Views/List/ObjectMixerEditorListRowData.h"
#include "Views/List/ObjectMixerUtils.h"
#include "Views/List/SObjectMixerEditorList.h"
#include "Views/Widgets/SInlinePropertyCellWidget.h"

#include "ISceneOutlinerMode.h"
#include "SortHelper.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "ObjectMixerEditor"

const TSharedRef<SWidget> FObjectMixerOutlinerVisibilityColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (FObjectMixerUtils::AsActorRow(TreeItem) || FObjectMixerUtils::AsFolderRow(TreeItem))
	{
		return FSceneOutlinerGutter::ConstructRowWidget(TreeItem, Row);
	}

	return SNullWidget::NullWidget;
}

FName FObjectMixerOutlinerSoloColumn::GetID()
{
	static FName ColumnName("Object Mixer Solo");
	return ColumnName;
}

FText FObjectMixerOutlinerSoloColumn::GetLocalizedColumnName()
{
	return LOCTEXT("SoloColumnName", "Solo");
}

SHeaderRow::FColumn::FArguments FObjectMixerOutlinerSoloColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(LOCTEXT("SoloColumnTooltip", "Solo Objects"))
		.HeaderComboVisibility(EHeaderComboVisibility::Never)
		.HeaderContentPadding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FObjectMixerEditorStyle::Get().GetBrush("ObjectMixer.Solo"))
		]
		.MenuContent()
		[
			StaticCastSharedPtr<SObjectMixerEditorList>(WeakOutliner.Pin())->GenerateHeaderRowContextMenu()
		];
}

const TSharedRef<SWidget> FObjectMixerOutlinerSoloColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (FObjectMixerUtils::AsActorRow(TreeItem) || FObjectMixerUtils::AsFolderRow(TreeItem))
	{
		TWeakPtr<ISceneOutlinerTreeItem> WeakTreeItem = TreeItem;
	
		return SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Visibility(EVisibility::SelfHitTestInvisible)
				.Padding(FMargin(2,0,0,0))
				[
					SNew(SImage)
					.ColorAndOpacity_Raw(this, &FObjectMixerOutlinerSoloColumn::GetForegroundColor, TreeItem, &Row)
					.Image(FObjectMixerEditorStyle::Get().GetBrush("ObjectMixer.Solo"))
					.OnMouseButtonDown_Lambda(
						[this, WeakTreeItem] (const FGeometry&, const FPointerEvent&)
						{
							if (WeakTreeItem.IsValid())
							{
								OnClickSoloIcon(WeakTreeItem.Pin().ToSharedRef());

							   return FReply::Handled();
							}

							return FReply::Unhandled();
						}
					)
				]
			;
	}
	
	return SNullWidget::NullWidget;
}

void FObjectMixerOutlinerSoloColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const
{
	FSceneOutlinerSortHelper<int32, bool>()
		/** Sort by type first */
		.Primary([this](const ISceneOutlinerTreeItem& Item){ return WeakOutliner.Pin()->GetMode()->GetTypeSortPriority(Item); }, SortMode)
		/** Then by solo state */
		.Secondary([](const ISceneOutlinerTreeItem& Item) {return FSceneOutlinerVisibilityCache().GetVisibility(Item); },	SortMode)
		.Sort(RootItems);
}

FSlateColor FObjectMixerOutlinerSoloColumn::GetForegroundColor(
	FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>* Row) const
{
	if (FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(TreeItem))
	{
		if (RowData->GetIsSelected(TreeItem) || RowData->GetRowSoloState())
		{
			return FSlateColor::UseForeground();
		}
		
		if (Row->IsHovered())
		{
			return FStyleColors::ForegroundHover;
		}
	}
		
	return FLinearColor::Transparent;	
}

void FObjectMixerOutlinerSoloColumn::OnClickSoloIcon(const FSceneOutlinerTreeItemRef RowPtr)
{
	if (FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(RowPtr))
	{
		if (TSharedPtr<SObjectMixerEditorList> ListView = RowData->GetListView().Pin())
		{
			const bool bNewSolo = !RowData->GetRowSoloState();
								
			using LambdaType = void(*)(const TSharedPtr<ISceneOutlinerTreeItem>&, const bool);
		
			static LambdaType SetSoloPerRowRecursively =
				[](const TSharedPtr<ISceneOutlinerTreeItem>& RowPtr, const bool bNewSolo)
				{
					if (FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(RowPtr))
					{
						if (bNewSolo)
						{
							RowData->SetUserHiddenInEditor(false);
						}
			
						RowData->SetRowSoloState(bNewSolo);
								
						for (const TWeakPtr<ISceneOutlinerTreeItem>& SelectedItem : RowPtr->GetChildren())
						{
							if (SelectedItem.IsValid())
							{
								SetSoloPerRowRecursively(SelectedItem.Pin(), bNewSolo);
							}
						}
					}
				};
								
			if (ListView->GetSelectedTreeViewItemCount() > 0 && RowData->GetIsSelected(RowPtr))
			{
				for (const TSharedPtr<ISceneOutlinerTreeItem>& SelectedItem : ListView->GetSelectedTreeViewItems())
				{
					SetSoloPerRowRecursively(SelectedItem, bNewSolo);
				}
			}
			else
			{
				SetSoloPerRowRecursively(RowPtr, bNewSolo);
			}

			{
				FScopedTransaction SoloObject(LOCTEXT("ChangeSoloState", "Solo Object"));
				ListView->EvaluateAndSetEditorVisibilityPerRow();
			}

		}
	}
}

FName FObjectMixerOutlinerPropertyColumn::GetID(const FProperty* InProperty)
{
	return FName("FObjectMixerOutlinerPropertyColumn_" + InProperty->GetName());
}

FText FObjectMixerOutlinerPropertyColumn::GetDisplayNameText(const FProperty* InProperty)
{
	if (InProperty)
	{
		return InProperty->GetDisplayNameText();
	}

	return FText::GetEmpty();
}

SHeaderRow::FColumn::FArguments FObjectMixerOutlinerPropertyColumn::ConstructHeaderRowColumn()
{
	const FText DisplayName = GetDisplayNameText(Property);
	
	return SHeaderRow::Column(GetColumnID())
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(DisplayName)
		.HeaderContentPadding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Text(DisplayName)
		];
}

const TSharedRef<SWidget> FObjectMixerOutlinerPropertyColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	const TWeakPtr<ISceneOutlinerTreeItem> WeakRowPtr = TreeItem;
	
	return SNew(SBox)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Visibility(EVisibility::SelfHitTestInvisible)
			[
				SNew(SInlinePropertyCellWidget, Property->GetFName(), TreeItem)
				.OnPropertyValueChanged(this, &FObjectMixerOutlinerPropertyColumn::OnPropertyChanged, WeakRowPtr)
			]
		;
}

void FObjectMixerOutlinerPropertyColumn::OnPropertyChanged(const FPropertyChangedEvent& Event, TWeakPtr<ISceneOutlinerTreeItem> WeakRowPtr) const
{
	if (const TSharedPtr<ISceneOutlinerTreeItem> TreeItem = WeakRowPtr.Pin())
	{
		if (FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(TreeItem))
		{
			if (const TSharedPtr<SObjectMixerEditorList> ListView = RowData->GetListView().Pin())
			{
				// If fewer than 2 items are selected, there is nothing to propagate. Early out.
				if (ListView->GetSelectedTreeViewItemCount() < 2)
				{
					return;
				}
				
				const EPropertyValueSetFlags::Type Flag =
					 Event.ChangeType & EPropertyChangeType::Interactive ?
						 EPropertyValueSetFlags::InteractiveChange : EPropertyValueSetFlags::DefaultFlags;

				FObjectMixerEditorListRowData::FPropertyPropagationInfo PropagationInfo; 
				PropagationInfo.RowIdentifier = TreeItem->GetID();
				PropagationInfo.PropertyName = Property->GetFName();
				PropagationInfo.PropertyValueSetFlags = Flag;
		
				if (Flag == EPropertyValueSetFlags::InteractiveChange)
				{
					RowData->PropagateChangesToSimilarSelectedRowProperties(TreeItem.ToSharedRef(), PropagationInfo);
				}
				else
				{
					// If not an interactive change, schedule property propagation on next frame
					ListView->AddToPendingPropertyPropagations(PropagationInfo);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
