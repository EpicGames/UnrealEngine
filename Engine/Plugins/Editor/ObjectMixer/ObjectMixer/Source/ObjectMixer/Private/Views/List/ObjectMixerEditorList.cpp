// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/ObjectMixerEditorList.h"

#include "ObjectMixerEditorLog.h"
#include "ObjectMixerEditorModule.h"
#include "ObjectMixerEditorSettings.h"
#include "ObjectMixerEditorSerializedData.h"
#include "SelectionInterface/LevelEditorObjectMixerSelectionInterface.h"

#include "Framework/Commands/GenericCommands.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "LevelEditorSequencerIntegration.h"
#include "Misc/TransactionObjectEvent.h"
#include "Modules/ModuleManager.h"
#include "Selection.h"
#include "Views/List/SObjectMixerEditorList.h"

void FObjectMixerEditorList::Initialize()
{
	RegisterAndMapContextMenuCommands();

	BindRefreshDelegates();

	// The base module is always notified when a blueprint filter is compiled.
	// When it is, all submodules should rebuild their lists.
	OnBlueprintFilterCompiledHandle = FObjectMixerEditorModule::Get().OnBlueprintFilterCompiled().AddLambda([this]()
	{
		CacheAndRebuildFilters(true);
	});
}

void FObjectMixerEditorList::RegisterAndMapContextMenuCommands()
{
	ObjectMixerElementEditCommands = MakeShared<FUICommandList>();

	ObjectMixerElementEditCommands->MapAction(FGenericCommands::Get().Cut,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("EDIT CUT") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Cut_CanExecute )
	);
	ObjectMixerElementEditCommands->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("EDIT COPY") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Copy_CanExecute )
	);
	ObjectMixerElementEditCommands->MapAction(FGenericCommands::Get().Paste,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("EDIT PASTE") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Paste_CanExecute )
	);
	ObjectMixerElementEditCommands->MapAction(FGenericCommands::Get().Duplicate,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("DUPLICATE") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Duplicate_CanExecute )
	);
	ObjectMixerElementEditCommands->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::ExecuteExecCommand, FString( TEXT("DELETE") ) ),
		FCanExecuteAction::CreateStatic( &FLevelEditorActionCallbacks::Delete_CanExecute )
	);
	ObjectMixerElementEditCommands->MapAction(FGenericCommands::Get().Rename,
		FUIAction(FExecuteAction::CreateRaw(this, &FObjectMixerEditorList::OnRenameCommand))
	);
}

void FObjectMixerEditorList::RebuildCollectionSelector()
{
	ListWidget->RebuildCollectionSelector();
}

void FObjectMixerEditorList::SetDefaultFilterClass(UClass* InNewClass)
{
	DefaultFilterClass = InNewClass;
	AddObjectFilterClass(InNewClass, false);
}

bool FObjectMixerEditorList::IsClassSelected(UClass* InClass) const
{
	return GetObjectFilterClasses().Contains(InClass);
}

const TArray<TObjectPtr<UObjectMixerObjectFilter>>& FObjectMixerEditorList::GetObjectFilterInstances()
{
	if (ObjectFilterInstances.Num() == 0)
	{
		CacheObjectFilterInstances();
	}

	return ObjectFilterInstances;
}

const UObjectMixerObjectFilter* FObjectMixerEditorList::GetMainObjectFilterInstance()
{
	return GetObjectFilterInstances().Num() > 0 ? GetObjectFilterInstances()[0].Get() : nullptr;
}

void FObjectMixerEditorList::CacheObjectFilterInstances()
{
	ObjectFilterInstances.Reset();

	if (ObjectFilterClasses.Num() > 0)
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : ObjectFilterClasses)
		{
			UObjectMixerObjectFilter* NewInstance = NewObject<UObjectMixerObjectFilter>(GetTransientPackage(), Class);
			ObjectFilterInstances.Add(TObjectPtr<UObjectMixerObjectFilter>(NewInstance));
		}

		BuildPerformanceCache();
	}
}

TSet<UClass*> FObjectMixerEditorList::ForceGetObjectClassesToFilter()
{
	TSet<UClass*> ReturnValue;
	for (const TObjectPtr<UObjectMixerObjectFilter>& Filter : GetObjectFilterInstances())
	{
		ReturnValue.Append(Filter->GetObjectClassesToFilter());
	}
		
	return ReturnValue;
}

TSet<TSubclassOf<AActor>> FObjectMixerEditorList::ForceGetObjectClassesToPlace()
{
	TSet<TSubclassOf<AActor>> ReturnValue;

	for (const TObjectPtr<UObjectMixerObjectFilter>& Filter : GetObjectFilterInstances())
	{
		ReturnValue.Append(Filter->GetObjectClassesToPlace());
	}
		
	return ReturnValue;
}

void FObjectMixerEditorList::AddObjectFilterClass(UClass* InObjectFilterClass, const bool bShouldRebuild)
{
	if (ensureAlwaysMsgf(InObjectFilterClass->IsChildOf(UObjectMixerObjectFilter::StaticClass()), TEXT("%hs: Class '%s' is not a child of UObjectMixerObjectFilter."), __FUNCTION__, *InObjectFilterClass->GetName()))
	{
		ObjectFilterClasses.AddUnique(InObjectFilterClass);

		CacheObjectFilterInstances();

		if (bShouldRebuild)
		{
			RequestRebuildList();
		}
	}
}

void FObjectMixerEditorList::RemoveObjectFilterClass(UClass* InObjectFilterClass, const bool bCacheAndRebuild)
{
	if (ObjectFilterClasses.Remove(InObjectFilterClass) > 0 && bCacheAndRebuild)
	{
		CacheAndRebuildFilters();
	}
}

FObjectMixerEditorModule* FObjectMixerEditorList::GetModulePtr() const
{
	return FModuleManager::GetModulePtr<FObjectMixerEditorModule>(GetModuleName());
}

UObjectMixerEditorSerializedData* FObjectMixerEditorList::GetSerializedData() const
{
	return GetMutableDefault<UObjectMixerEditorSerializedData>();
}

bool FObjectMixerEditorList::RequestAddObjectsToCollection(const FName& CollectionName, const TSet<FSoftObjectPath>& ObjectsToAdd) const
{
	if (UObjectMixerEditorSerializedData* SerializedData = GetSerializedData())
	{
		if (auto It = GetObjectFilterClasses().CreateConstIterator(); It)
		{
			const TSubclassOf<UObjectMixerObjectFilter>& Class = *It;
			const FName FilterName = Class->GetFName();
			if (SerializedData->AddObjectsToCollection(FilterName, CollectionName, ObjectsToAdd))
			{
				if (IsCollectionSelected(CollectionName))
				{
					RequestRebuildList();
				}
				return true;
			}
			return false;
		}
	}

	return false;
}

bool FObjectMixerEditorList::RequestRemoveObjectsFromCollection(const FName& CollectionName, const TSet<FSoftObjectPath>& ObjectsToRemove) const
{
	if (UObjectMixerEditorSerializedData* SerializedData = GetSerializedData())
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : GetObjectFilterClasses())
		{
			const FName FilterName = Class->GetFName();
			if (SerializedData->DoesCollectionExist(FilterName, CollectionName))
			{
				if (SerializedData->RemoveObjectsFromCollection(FilterName, CollectionName, ObjectsToRemove))
				{
					if (IsCollectionSelected(CollectionName))
					{
						RequestRebuildList();
					}
					return true;
				}
				return false;
			}
		}
	}

	return false;
}

bool FObjectMixerEditorList::RequestRemoveCollection(const FName& CollectionName) const
{
	if (UObjectMixerEditorSerializedData* SerializedData = GetSerializedData())
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : GetObjectFilterClasses())
		{
			const FName FilterName = Class->GetFName();
			if (SerializedData->DoesCollectionExist(FilterName, CollectionName))
			{
				return SerializedData->RemoveCollection(FilterName, CollectionName);
			}
		}
	}

	return false;
}

bool FObjectMixerEditorList::RequestDuplicateCollection(const FName& CollectionToDuplicateName, FName& DesiredDuplicateName) const
{
	if (UObjectMixerEditorSerializedData* SerializedData = GetSerializedData())
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : GetObjectFilterClasses())
		{
			const FName FilterName = Class->GetFName();
			if (SerializedData->DoesCollectionExist(FilterName, CollectionToDuplicateName))
			{
				return SerializedData->DuplicateCollection(FilterName, CollectionToDuplicateName, DesiredDuplicateName);
			}
		}
	}

	return false;
}

bool FObjectMixerEditorList::RequestReorderCollection(const FName& CollectionToMoveName, const FName& CollectionInsertBeforeName) const
{
	if (UObjectMixerEditorSerializedData* SerializedData = GetSerializedData())
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : GetObjectFilterClasses())
		{
			const FName FilterName = Class->GetFName();
			if (SerializedData->DoesCollectionExist(FilterName, CollectionToMoveName))
			{
				return SerializedData->ReorderCollection(FilterName, CollectionToMoveName, CollectionInsertBeforeName);
			}
		}
	}

	return false;
}

bool FObjectMixerEditorList::RequestRenameCollection(const FName& CollectionNameToRename,
	const FName& NewCollectionName) const
{
	if (UObjectMixerEditorSerializedData* SerializedData = GetSerializedData())
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : GetObjectFilterClasses())
		{
			const FName FilterName = Class->GetFName();
			if (SerializedData->DoesCollectionExist(FilterName, CollectionNameToRename))
			{
				return SerializedData->RenameCollection(FilterName, CollectionNameToRename, NewCollectionName);
			}
		}
	}

	return false;
}

bool FObjectMixerEditorList::DoesCollectionExist(const FName& CollectionName) const
{
	if (UObjectMixerEditorSerializedData* SerializedData = GetSerializedData())
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : GetObjectFilterClasses())
		{
			const FName FilterName = Class->GetFName();
			if (SerializedData->DoesCollectionExist(FilterName, CollectionName))
			{
				return true;
			}
		}
	}

	return false;
}

bool FObjectMixerEditorList::IsObjectInCollection(const FName& CollectionName, const FSoftObjectPath& InObject) const
{
	if (UObjectMixerEditorSerializedData* SerializedData = GetSerializedData())
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : GetObjectFilterClasses())
		{
			const FName FilterName = Class->GetFName();
			if (SerializedData->DoesCollectionExist(FilterName, CollectionName))
			{
				return SerializedData->IsObjectInCollection(FilterName, CollectionName, InObject);
			}
		}
	}

	return false;
}

TSet<FName> FObjectMixerEditorList::GetCollectionsForObject(const FSoftObjectPath& InObject) const
{
	TSet<FName> ReturnValue;
	if (UObjectMixerEditorSerializedData* SerializedData = GetSerializedData())
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : GetObjectFilterClasses())
		{
			const FName FilterName = Class->GetFName();
			ReturnValue.Append(SerializedData->GetCollectionsForObject(FilterName, InObject));
		}
	}

	return ReturnValue;
}

TArray<FName> FObjectMixerEditorList::GetAllCollectionNames() const
{
	TArray<FName> ReturnValue;
	if (UObjectMixerEditorSerializedData* SerializedData = GetSerializedData())
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : GetObjectFilterClasses())
		{
			const FName FilterName = Class->GetFName();
			ReturnValue.Append(SerializedData->GetAllCollectionNames(FilterName));
		}
	}

	return ReturnValue;
}

const TSubclassOf<UObjectMixerObjectFilter>& FObjectMixerEditorList::GetDefaultFilterClass() const
{
	return DefaultFilterClass;
}

FObjectMixerEditorList::FObjectMixerEditorList(const FName InModuleName, TSharedPtr<IObjectMixerSelectionInterface> InSelectionInterface)
{
	ModuleName = InModuleName;
	SelectionInterface = InSelectionInterface.IsValid()
		? InSelectionInterface.ToSharedRef()
		: MakeShared<FLevelEditorObjectMixerSelectionInterface>();
	
	OnPostFilterChangeDelegate.AddRaw(this, &FObjectMixerEditorList::OnPostFilterChange);
}

FObjectMixerEditorList::~FObjectMixerEditorList()
{
	FlushWidget();

	UnbindRefreshDelegates();

	if (FObjectMixerEditorModule* ObjectMixerEditorModule = FModuleManager::GetModulePtr<FObjectMixerEditorModule>("ObjectMixer"))
	{
		ObjectMixerEditorModule->OnBlueprintFilterCompiled().Remove(OnBlueprintFilterCompiledHandle);
	}
}

void FObjectMixerEditorList::FlushWidget()
{
	OnPreFilterChangeDelegate.RemoveAll(this);
	OnPostFilterChangeDelegate.RemoveAll(this);
	
	ListWidget.Reset();
}

TSharedRef<SWidget> FObjectMixerEditorList::GetOrCreateWidget()
{
	if (!ListWidget.IsValid())
	{
		CreateWidget();
	}

	RequestRebuildList();

	return ListWidget.ToSharedRef();
}

TSharedRef<SWidget> FObjectMixerEditorList::CreateWidget()
{
	return SAssignNew(ListWidget, SObjectMixerEditorList, SharedThis(this));
}

void FObjectMixerEditorList::RequestRegenerateListWidget()
{
	if (FObjectMixerEditorModule* Module = GetModulePtr())
	{
		Module->RegenerateListWidget();
	}
}

void FObjectMixerEditorList::OnPostFilterChange()
{
	RequestRebuildList();
}

void FObjectMixerEditorList::RequestRebuildList() const
{
	if (ListWidget.IsValid())
	{
		ListWidget->RequestRebuildList();
	}
}

void FObjectMixerEditorList::RefreshList() const
{
	if (ListWidget.IsValid())
	{
		ListWidget->RefreshList();
	}
}

void FObjectMixerEditorList::BuildPerformanceCache()
{
	TSet<UClass*> LocalObjectClassesToFilterCache;
	TSet<FName> LocalColumnsToShowByDefaultCache;
	TSet<FName> LocalColumnsToExcludeCache;
	TSet<FName> LocalForceAddedColumnsCache;

	bShouldIncludeUnsupportedPropertiesCache = false;
	bShouldShowTransientObjectsCache = false;

	// Aggregate caches - append everything to all-inclusive lists
	for (const TObjectPtr<UObjectMixerObjectFilter>& FilterInstance : GetObjectFilterInstances())
	{
		if (!FilterInstance)
		{
			UE_LOGF(LogObjectMixerEditor, Display, "%s: UObjectMixerObjectFilter instance not valid.", __FUNCTION__);
			continue;
		}

		LocalObjectClassesToFilterCache.Append(FilterInstance->GetObjectClassesToFilter());
		LocalColumnsToShowByDefaultCache.Append(FilterInstance->GetColumnsToShowByDefault());
		LocalColumnsToExcludeCache.Append(FilterInstance->GetColumnsToExclude());
		LocalForceAddedColumnsCache.Append(FilterInstance->GetForceAddedColumns());

		// These are zero-sum - if they're true for any filter, they must remain true
		if (!bShouldIncludeUnsupportedPropertiesCache)
		{
			bShouldIncludeUnsupportedPropertiesCache = FilterInstance->ShouldIncludeUnsupportedProperties();
		}
		if (!bShouldShowTransientObjectsCache)
		{
			bShouldShowTransientObjectsCache = FilterInstance->GetShowTransientObjects();
		}
	}

	if (LocalObjectClassesToFilterCache.Difference(ObjectClassesToFilterCache).Num() > 0 || 
		ObjectClassesToFilterCache.Difference(LocalObjectClassesToFilterCache).Num() > 0)
	{
		ObjectClassesToFilterCache = LocalObjectClassesToFilterCache;
	}

	if (LocalColumnsToShowByDefaultCache.Difference(ColumnsToShowByDefaultCache).Num() > 0 || 
		ColumnsToShowByDefaultCache.Difference(LocalColumnsToShowByDefaultCache).Num() > 0)
	{
		ColumnsToShowByDefaultCache = LocalColumnsToShowByDefaultCache;
	}

	if (LocalColumnsToExcludeCache.Difference(ColumnsToExcludeCache).Num() > 0 || 
		ColumnsToExcludeCache.Difference(LocalColumnsToExcludeCache).Num() > 0)
	{
		ColumnsToExcludeCache = LocalColumnsToExcludeCache;
	}

	if (LocalForceAddedColumnsCache.Difference(ForceAddedColumnsCache).Num() > 0 || 
		ForceAddedColumnsCache.Difference(LocalForceAddedColumnsCache).Num() > 0)
	{
		ForceAddedColumnsCache = LocalForceAddedColumnsCache;
	}

	// These properties should be governed by the Main instance
	if (const UObjectMixerObjectFilter* FilterInstance = GetMainObjectFilterInstance())
	{
		if (const EObjectMixerInheritanceInclusionOptions PropertyInheritanceInclusionOptions =
			FilterInstance->GetObjectMixerPropertyInheritanceInclusionOptions(); 
			PropertyInheritanceInclusionOptions != PropertyInheritanceInclusionOptionsCache)
		{
			PropertyInheritanceInclusionOptionsCache = PropertyInheritanceInclusionOptions;
		}
	}
	else
	{
		UE_LOGF(LogObjectMixerEditor, Display, "%s: No Main UObjectMixerObjectFilter instance found.", __FUNCTION__);
	}
}

bool FObjectMixerEditorList::ShouldShowTransientObjects() const
{
	return bShouldShowTransientObjectsCache;
}

void FObjectMixerEditorList::OnRenameCommand()
{
	if (ListWidget.IsValid())
	{
		ListWidget->OnRenameCommand();;
	}
}

TArray<TSharedPtr<ISceneOutlinerTreeItem>> FObjectMixerEditorList::GetSelectedTreeViewItems() const
{
	if (ListWidget.IsValid())
	{
		return ListWidget->GetSelectedTreeViewItems();
	}

	return {};
}

int32 FObjectMixerEditorList::GetSelectedTreeViewItemCount() const
{
	if (ListWidget.IsValid())
	{
		return ListWidget->GetSelectedTreeViewItemCount();
	}

	return INDEX_NONE;
}

TSet<TSharedPtr<ISceneOutlinerTreeItem>> FObjectMixerEditorList::GetSoloRows() const
{
	if (ListWidget.IsValid())
	{
		return ListWidget->GetSoloRows();
	}
	
	return {};
}

void FObjectMixerEditorList::ClearSoloRows()
{
	if (ListWidget.IsValid())
	{
		ListWidget->ClearSoloRows();
	}
}

bool FObjectMixerEditorList::IsListInSoloState() const
{
	if (ListWidget.IsValid())
	{
		return ListWidget->IsListInSoloState();
	}
	
	return false;
}

void FObjectMixerEditorList::EvaluateAndSetEditorVisibilityPerRow()
{
	if (ListWidget.IsValid())
	{
		ListWidget->EvaluateAndSetEditorVisibilityPerRow();
	}
}

const TSet<FName>& FObjectMixerEditorList::GetSelectedCollections() const
{
	return SelectedCollections;
}

bool FObjectMixerEditorList::IsCollectionSelected(const FName& CollectionName) const
{
	return GetSelectedCollections().Contains(CollectionName);
}

void FObjectMixerEditorList::SetSelectedCollections(const TSet<FName> InSelectedCollections)
{
	SelectedCollections = InSelectedCollections;
}

void FObjectMixerEditorList::SetCollectionSelected(const FName& CollectionName, const bool bNewSelected)
{
	if (UObjectMixerEditorSerializedData* SerializedData = GetMutableDefault<UObjectMixerEditorSerializedData>())
	{
		for (const TSubclassOf<UObjectMixerObjectFilter>& Class : GetObjectFilterClasses())
		{
			const FName FilterName = Class->GetFName();
			if (SerializedData->DoesCollectionExist(FilterName, CollectionName))
			{
				if (bNewSelected)
				{
					SelectedCollections.Add(CollectionName);
				}
				else
				{
					SelectedCollections.Remove(CollectionName);
				}

				return;
			}
		}
	}
}

void FObjectMixerEditorList::BindRefreshDelegates()
{
	check(GEngine && GEditor);
	DelegateHandles.Add(GEngine->OnLevelActorListChanged().AddLambda([this]()
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(GEngine->OnActorFolderAdded().AddLambda([this](UActorFolder*)
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(GEngine->OnActorFoldersUpdatedEvent().AddLambda([this](ULevel*)
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(GEngine->OnLevelActorFolderChanged().AddLambda([this](const AActor*, FName)
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(GEngine->OnLevelActorAttached().AddLambda([this](AActor*, const AActor*)
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(GEngine->OnLevelActorDetached().AddLambda([this](AActor*, const AActor*)
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(GEditor->OnLevelActorAdded().AddLambda([this](AActor*)
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(GEditor->OnLevelActorDeleted().AddLambda([this](AActor*)
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(FEditorDelegates::MapChange.AddLambda([this](uint32)
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(FEditorDelegates::PostUndoRedo.AddLambda([this]()
	{
		RequestRebuildList();

		// Because we can undo/redo collection adds and removes, we need to save the data after each
		GetMutableDefault<UObjectMixerEditorSerializedData>()->SaveConfig();
	}));

	DelegateHandles.Add(FCoreUObjectDelegates::OnObjectTransacted.AddLambda([this](UObject*, const FTransactionObjectEvent& Event)
	{
		if (Event.GetEventType() == ETransactionObjectEventType::Finalized)
		{
			if (Event.HasNameChange())
			{
				RequestRebuildList();
			}
			else if (const TSet<FName> PropertiesThatRequireRefresh = GetPropertiesThatRequireRefresh(); PropertiesThatRequireRefresh.Num() > 0)
			{
				const TSet<FName> ChangedPropertyNames = TSet<FName>(Event.GetChangedProperties());

				if (ChangedPropertyNames.Intersect(PropertiesThatRequireRefresh).Num() > 0)
				{
					RequestRebuildList();
				}
			}
		}
	}));

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	DelegateHandles.Add(LevelEditorModule.OnComponentsEdited().AddLambda([this]()
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(FLevelEditorSequencerIntegration::Get().GetOnSequencersChanged().AddLambda([this]
	{
		RequestRebuildList();
	}));

	DelegateHandles.Add(GetMutableDefault<UObjectMixerEditorSettings>()->OnSettingChanged().AddLambda(
	[this](UObject*, FPropertyChangedEvent& Event)
	{
		if (UObjectMixerEditorSettings::DoesPropertyChangeRequireListRebuild(Event))
		{
			RequestRebuildList();
		}
	}));
}

void FObjectMixerEditorList::UnbindRefreshDelegates()
{
	// Unbind Delegates
	if (GEngine)
	{
		GEngine->OnLevelActorFolderChanged().RemoveAll(this);
		GEngine->OnLevelActorListChanged().RemoveAll(this);
		GEngine->OnActorFolderAdded().RemoveAll(this);
		GEngine->OnLevelActorDetached().RemoveAll(this);
		GEngine->OnLevelActorAttached().RemoveAll(this);
		GEngine->OnActorFolderAdded().RemoveAll(this);
		GEngine->OnActorFoldersUpdatedEvent().RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->OnLevelActorAdded().RemoveAll(this);
		GEditor->OnLevelActorDeleted().RemoveAll(this);
		GEditor->GetSelectedActors()->SelectionChangedEvent.RemoveAll(this);
	}

	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::PostUndoRedo.RemoveAll(this);
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);

	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditorModule->OnComponentsEdited().RemoveAll(this);
	}

	FLevelEditorSequencerIntegration::Get().GetOnSequencersChanged().RemoveAll(this);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("ObjectMixer")))
	{
		if (UObjectMixerEditorSettings* Settings = GetMutableDefault<UObjectMixerEditorSettings>())
		{
			Settings->OnSettingChanged().RemoveAll(this);
		}
	}

	for (FDelegateHandle& Delegate : DelegateHandles)
	{
		Delegate.Reset();
	}
	DelegateHandles.Empty();
}

TSet<FName> FObjectMixerEditorList::GetPropertiesThatRequireRefresh()
{
	TSet<FName> ReturnValue;

	for (const TObjectPtr<UObjectMixerObjectFilter>& Instance : GetObjectFilterInstances())
	{
		ReturnValue.Append(Instance->GetPropertiesThatRequireListRefresh());
	}

	return ReturnValue;
}