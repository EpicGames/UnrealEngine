// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/ObjectMixerEditorListRowData.h"

#include "ObjectMixerEditorLog.h"
#include "Views/List/ObjectMixerUtils.h"
#include "Views/List/SObjectMixerEditorList.h"

#include "Editor.h"
#include "GameFramework/Actor.h"
#include "ScopedTransaction.h"

const TArray<TObjectPtr<UObjectMixerObjectFilter>>& FObjectMixerEditorListRowData::GetObjectFilterInstances() const
{
	TSharedPtr<SObjectMixerEditorList> ListView = GetListView().Pin();
	check (ListView.IsValid());

	const TSharedPtr<FObjectMixerEditorList> PinnedListModel = ListView->GetListModelPtr().Pin();
	check (PinnedListModel);
	
	return PinnedListModel->GetObjectFilterInstances();
}

const UObjectMixerObjectFilter* FObjectMixerEditorListRowData::GetMainObjectFilterInstance() const
{
	if (TSharedPtr<SObjectMixerEditorList> ListView = GetListView().Pin(); ListView.IsValid())
	{
		if (const TSharedPtr<FObjectMixerEditorList> PinnedListModel = ListView->GetListModelPtr().Pin())
		{
			return PinnedListModel->GetMainObjectFilterInstance();
		}
	}
	
	return nullptr;
}

bool FObjectMixerEditorListRowData::GetIsTreeViewItemExpanded(const TSharedRef<ISceneOutlinerTreeItem> InRow)
{
	if (TSharedPtr<SObjectMixerEditorList> ListView = GetListView().Pin(); ListView.IsValid())
	{
		return ListView->IsTreeViewItemExpanded(InRow);
	}

	return false;
}

void FObjectMixerEditorListRowData::SetIsTreeViewItemExpanded(const TSharedRef<ISceneOutlinerTreeItem> InRow, const bool bNewExpanded)
{
	if (TSharedPtr<SObjectMixerEditorList> ListView = GetListView().Pin(); ListView.IsValid())
	{
		ListView->SetTreeViewItemExpanded(InRow, bNewExpanded);
	}
}

bool FObjectMixerEditorListRowData::GetIsSelected(const TSharedRef<ISceneOutlinerTreeItem> InRow)
{
	if (TSharedPtr<SObjectMixerEditorList> ListView = GetListView().Pin(); ListView.IsValid())
	{
		return ListView->IsTreeViewItemSelected(InRow);
	}

	return false;
}

void FObjectMixerEditorListRowData::SetIsSelected(const TSharedRef<ISceneOutlinerTreeItem> InRow, const bool bNewSelected)
{
	if (TSharedPtr<SObjectMixerEditorList> ListView = GetListView().Pin(); ListView.IsValid())
	{
		return ListView->SetTreeViewItemSelected(InRow, bNewSelected);
	}
}

bool FObjectMixerEditorListRowData::HasAtLeastOneChildThatIsNotSolo(const TSharedRef<ISceneOutlinerTreeItem> InRow, const bool bRecursive) const
{
	for (const TWeakPtr<ISceneOutlinerTreeItem>& ChildRow : InRow->GetChildren())
	{
		if (const TSharedPtr<ISceneOutlinerTreeItem> PinnedChildRow = ChildRow.Pin())
		{
			if (!FObjectMixerUtils::GetRowData(PinnedChildRow)->GetRowSoloState())
			{
				return true;
			}

			if (bRecursive && FObjectMixerUtils::GetRowData(PinnedChildRow)->HasAtLeastOneChildThatIsNotSolo(PinnedChildRow.ToSharedRef(), true))
			{
				return true;
			}
		}
	}

	return false;
}

TWeakPtr<SObjectMixerEditorList> FObjectMixerEditorListRowData::GetListView() const
{
	if (SceneOutlinerPtr.IsValid())
	{
		return StaticCastSharedPtr<SObjectMixerEditorList>(SceneOutlinerPtr.Pin());
	}

	return nullptr;
}

TArray<TSharedPtr<ISceneOutlinerTreeItem>> FObjectMixerEditorListRowData::GetSelectedTreeViewItems() const
{
	if (TSharedPtr<SObjectMixerEditorList> ListView = GetListView().Pin(); ListView.IsValid())
	{
		return ListView->GetSelectedTreeViewItems();
	}

	return {};
}

void FObjectMixerEditorListRowData::OnChangeVisibility(const FSceneOutlinerTreeItemRef TreeItem, const bool bNewVisible)
{
	if (FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(TreeItem))
	{
		if (TSharedPtr<SObjectMixerEditorList> ListView = GetListView().Pin(); ListView.IsValid())
		{
			ListView->ClearSoloRows();
		}

		RowData->SetUserHiddenInEditor(!bNewVisible);
	}
}

bool FObjectMixerEditorListRowData::IsUserSetHiddenInEditor() const
{
	return VisibilityRules.bShouldBeHiddenInEditor;
}

void FObjectMixerEditorListRowData::SetUserHiddenInEditor(const bool bNewHidden)
{
	VisibilityRules.bShouldBeHiddenInEditor = bNewHidden;
}

bool FObjectMixerEditorListRowData::GetRowSoloState() const
{
	return VisibilityRules.bShouldBeSolo;
}

void FObjectMixerEditorListRowData::SetRowSoloState(const bool bNewSolo)
{
	VisibilityRules.bShouldBeSolo = bNewSolo;
}

void FObjectMixerEditorListRowData::ClearSoloRows() const
{
	if (TSharedPtr<SObjectMixerEditorList> ListView = GetListView().Pin(); ListView.IsValid())
	{
		ListView->ClearSoloRows();
	}
}
DECLARE_STATS_GROUP(TEXT("ObjectMixer"), STATGROUP_ObjectMixer, STATCAT_Advanced);
void SetValueOnSelectedItems(
	const FString& ValueAsString, const TArray<TSharedPtr<ISceneOutlinerTreeItem>>& OtherSelectedItems,
	const FName& PropertyName, const TSharedPtr<ISceneOutlinerTreeItem> PinnedItem,
	const EPropertyValueSetFlags::Type InFlags)
{
	if (!ValueAsString.IsEmpty())
	{
		// Skip transactions on interactive and explicitly non-transactable value set
		const bool bShouldTransact = !(InFlags & EPropertyValueSetFlags::NotTransactable) && !(InFlags & EPropertyValueSetFlags::InteractiveChange);

		if (bShouldTransact && GEditor->CanTransact() && !GEditor->IsTransactionActive())
		{
			GEditor->BeginTransaction(
			   NSLOCTEXT("ObjectMixerEditor","BulkOnPropertyChangedTransaction", "Object Mixer - Bulk Edit Selected Row Properties"));
		}
		
		for (const TSharedPtr<ISceneOutlinerTreeItem>& SelectedRow : OtherSelectedItems)
		{
			if (SelectedRow == PinnedItem)
			{
				continue;
			}

			// Skip folders
			if (FObjectMixerUtils::AsFolderRow(SelectedRow))
			{
				continue;
			}

			UObject* ObjectToModify = FObjectMixerUtils::GetRowObject(SelectedRow, true);
			
			if (!IsValid(ObjectToModify))
			{
				UE_LOGF(LogObjectMixerEditor, Warning, "%s: Row '%ls' has no valid associated object to modify.", __FUNCTION__, *SelectedRow->GetDisplayString());
				continue;
			}

			// Use handles if valid, otherwise use ImportText. Need to use the handles to ensure the Blueprints update properly.
			if (FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(SelectedRow))
			{
				// Transactions are handled automatically by the handles, so no need to start a new transaction.
				if (const TWeakPtr<IPropertyHandle>* SelectedHandlePtr = RowData->PropertyNamesToHandles.Find(PropertyName))
				{
					if (SelectedHandlePtr->IsValid())
					{
						bool bShouldProceed = true;

						// If we need to transact, there's a possibility objects have been reconstructed
						// so we need to ensure we have handles matching ObjectToModify
						if (bShouldTransact)
						{
							TArray<UObject*> OuterObjects;
							SelectedHandlePtr->Pin()->GetOuterObjects(OuterObjects);
							bShouldProceed = OuterObjects.Contains(ObjectToModify);
						}
						
						if (bShouldProceed)
						{
							SelectedHandlePtr->Pin()->SetValueFromFormattedString(ValueAsString, InFlags);
							continue;
						}
					}
				}
			}

			// Handles approach failed, so use ImportText
			if (FProperty* PropertyToChange = FindFProperty<FProperty>(ObjectToModify->GetClass(), PropertyName))
			{
				if (void* ValuePtr = PropertyToChange->ContainerPtrToValuePtr<void>(ObjectToModify))
				{
					if (bShouldTransact)
					{
						ObjectToModify->Modify();
					}

					// Set the actual property value
					EPropertyChangeType::Type ChangeType =
						InFlags == EPropertyValueSetFlags::InteractiveChange
							? EPropertyChangeType::Interactive
							: EPropertyChangeType::ValueSet;

					// Set the actual property value and propagate to outer chain
					PropertyToChange->ImportText_Direct(*ValueAsString, ValuePtr, ObjectToModify, PPF_None);

					TArray<UObject*> ObjectsToModify = {ObjectToModify};
					UObject* Outer = ObjectToModify->GetOuter();
					while (Outer) 
					{
						ObjectsToModify.Add(Outer);

						Outer = Outer->GetOuter();
					}
					
					FPropertyChangedEvent ChangeEvent(
						PropertyToChange,
						ChangeType,
						MakeArrayView(ObjectsToModify));
					ObjectToModify->PostEditChangeProperty(ChangeEvent);
				}
			}
		}
		
		if (bShouldTransact)
		{
			GEditor->EndTransaction();
		}
	}
}

bool FObjectMixerEditorListRowData::PropagateChangesToSimilarSelectedRowProperties(
	const TSharedRef<ISceneOutlinerTreeItem> InRow, const FPropertyPropagationInfo PropertyPropagationInfo)
{
	if (PropertyPropagationInfo.PropertyName == NAME_None)
	{
		return true;
	}

	if (!GetIsSelected(InRow))
	{
		return true;
	}

	FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(InRow);
	
	const TWeakPtr<IPropertyHandle>* HandlePtr = RowData->PropertyNamesToHandles.Find(PropertyPropagationInfo.PropertyName);
	if (HandlePtr && HandlePtr->IsValid())
	{
		const TArray<TSharedPtr<ISceneOutlinerTreeItem>> OtherSelectedItems = RowData->GetSelectedTreeViewItems();
		if (OtherSelectedItems.Num())
		{
			FString ValueAsString;
			(*HandlePtr).Pin()->GetValueAsFormattedString(ValueAsString);
		
			SetValueOnSelectedItems(
				ValueAsString, OtherSelectedItems, PropertyPropagationInfo.PropertyName,
				InRow, PropertyPropagationInfo.PropertyValueSetFlags);
		}
		return true;
	}

	return false;
}

const FObjectMixerEditorListRowData::FTransientEditorVisibilityRules& FObjectMixerEditorListRowData::GetVisibilityRules() const
{
	return VisibilityRules;
}

void FObjectMixerEditorListRowData::SetVisibilityRules(const FTransientEditorVisibilityRules& InVisibilityRules)
{
	VisibilityRules = InVisibilityRules;
}
