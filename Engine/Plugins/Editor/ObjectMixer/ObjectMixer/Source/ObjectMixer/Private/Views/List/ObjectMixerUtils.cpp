// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/ObjectMixerUtils.h"

#include "ObjectMixerEditorSerializedData.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowActor.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowComponent.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowFolder.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowUObject.h"
#include "Views/List/SObjectMixerEditorList.h"

FObjectMixerEditorListRowFolder* FObjectMixerUtils::AsFolderRow(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem)
{
	check (InTreeItem);
	return InTreeItem->CastTo<FObjectMixerEditorListRowFolder>();
}

FObjectMixerEditorListRowActor* FObjectMixerUtils::AsActorRow(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem)
{
	check (InTreeItem);
	return InTreeItem->CastTo<FObjectMixerEditorListRowActor>();
}

FObjectMixerEditorListRowComponent* FObjectMixerUtils::AsComponentRow(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem)
{
	check (InTreeItem);
	return InTreeItem->CastTo<FObjectMixerEditorListRowComponent>();
}

FObjectMixerEditorListRowUObject* FObjectMixerUtils::AsObjectRow(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem)
{
	check (InTreeItem);
	return InTreeItem->CastTo<FObjectMixerEditorListRowUObject>();
}

FObjectMixerEditorListRowData* FObjectMixerUtils::GetRowData(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem)
{
	if (FObjectMixerEditorListRowFolder* AsFolder = AsFolderRow(InTreeItem))
	{
		if (AsFolder->RowData.IsValid())
		{
			return &AsFolder->RowData;
		}
	}

	if (FObjectMixerEditorListRowActor* AsActor = AsActorRow(InTreeItem))
	{
		if (AsActor->RowData.IsValid())
		{
			return &AsActor->RowData;
		}
	}

	if (FObjectMixerEditorListRowComponent* AsComponent = AsComponentRow(InTreeItem))
	{
		if (AsComponent->RowData.IsValid())
		{
			return &AsComponent->RowData;
		}
	}

	if (FObjectMixerEditorListRowUObject* AsObject = AsObjectRow(InTreeItem))
	{
		if (AsObject->RowData.IsValid())
		{
			return &AsObject->RowData;
		}
	}

	return nullptr;
}

UObject* FObjectMixerUtils::GetRowObject(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem, const bool bGetHybridRowComponent)
{	
	if (const FObjectMixerEditorListRowActor* ActorRow = AsActorRow(InTreeItem))
	{
		if (bGetHybridRowComponent && ActorRow->RowData.GetIsHybridRow())
		{
			return ActorRow->RowData.GetHybridComponent();
		}

		if (ActorRow->Actor.IsValid() && !ActorRow->Actor.IsStale())
		{
			return ActorRow->Actor.Get();
		}

		return ActorRow->OriginalObjectSoftPtr.Get();
	}

	if (const FObjectMixerEditorListRowComponent* ComponentRow = AsComponentRow(InTreeItem))
	{
		if (ComponentRow->Component.IsValid() && !ComponentRow->Component.IsStale())
		{
			return ComponentRow->Component.Get();
		}

		return ComponentRow->OriginalObjectSoftPtr.Get();
	}

	if (const FObjectMixerEditorListRowUObject* ObjectRow = AsObjectRow(InTreeItem))
	{
		return ObjectRow->ObjectSoftPtr.Get();
	}

	return nullptr;
}

AActor* FObjectMixerUtils::GetSelfOrOuterAsActor(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem)
{
	if (UObject* Object = GetRowObject(InTreeItem))
	{
		AActor* Actor = Cast<AActor>(Object);

		if (!Actor)
		{
			Actor = Object->GetTypedOuter<AActor>();
		}

		return Actor;
	}

	return nullptr;
}

bool FObjectMixerUtils::IsObjectRefInCollection(const FName& CollectionName, const UObject* Object, const TSharedPtr<FObjectMixerEditorList> ListModel)
{	
	if (Object)
	{
		if (CollectionName == UObjectMixerEditorSerializedData::AllCollectionName)
		{
			return true;
		}

		return ListModel->IsObjectInCollection(CollectionName, Object);
	}
	
	return false;
}

bool FObjectMixerUtils::IsObjectRefInCollection(const FName& CollectionName, TSharedPtr<ISceneOutlinerTreeItem> InTreeItem)
{	
	if (const UObject* Object = GetRowObject(InTreeItem))
	{
		if (TSharedPtr<SObjectMixerEditorList> ListView = GetRowData(InTreeItem)->GetListView().Pin())
		{
			if (const TSharedPtr<FObjectMixerEditorList> ListModel = ListView->GetListModelPtr().Pin())
			{
				return IsObjectRefInCollection(CollectionName, Object, ListModel);
			}
		}
	}
	
	return false;
}

void FObjectMixerUtils::SetChildRowsSelected(
	TSharedPtr<ISceneOutlinerTreeItem> InTreeItem, const bool bNewSelected, const bool bRecursive)
{
	for (const TWeakPtr<ISceneOutlinerTreeItem>& ChildRow : InTreeItem->GetChildren())
	{
		if (const TSharedPtr<ISceneOutlinerTreeItem> PinnedChildRow = ChildRow.Pin())
		{
			// Recurse even if not visible
			if (bRecursive)
			{
				SetChildRowsSelected(PinnedChildRow, bNewSelected, bRecursive);
			}
	
			GetRowData(PinnedChildRow)->SetIsSelected(PinnedChildRow.ToSharedRef(), bNewSelected);
		}
	}
}
