// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/RowTypes/ObjectMixerEditorListRowActor.h"

#include "Views/List/SObjectMixerEditorList.h"

#include "ScopedTransaction.h"
#include "GameFramework/Actor.h"

const FSceneOutlinerTreeItemType FObjectMixerEditorListRowActor::Type(&FActorTreeItem::Type);

void FObjectMixerEditorListRowActor::OnVisibilityChanged(const bool bNewVisibility)
{
	RowData.OnChangeVisibility(SharedThis(this), bNewVisibility);

	if (TSharedPtr<SObjectMixerEditorList> ListView = RowData.GetListView().Pin())
	{
		ListView->EvaluateAndSetEditorVisibilityPerRow();
	}
}

FSceneOutlinerTreeItemID FObjectMixerEditorListRowActor::GetID() const
{
	if (!OverrideParent.IsValid())
	{
		return FActorTreeItem::GetID();
	}

	// The same actor can appear as the child of multiple override parents, so generate unique IDs to let it display separately under each parent
	return FSceneOutlinerTreeItemID(FGuid::Combine(GetGuid(), OverrideParent.Get()->GetActorGuid()));
}
