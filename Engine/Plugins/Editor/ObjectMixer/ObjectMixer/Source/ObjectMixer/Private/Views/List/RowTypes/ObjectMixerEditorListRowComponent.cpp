// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/RowTypes/ObjectMixerEditorListRowComponent.h"

const FSceneOutlinerTreeItemType FObjectMixerEditorListRowComponent::Type(&FComponentTreeItem::Type);
