// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/RowTypes/ObjectMixerEditorListRowContainer.h"

#include "ISceneOutliner.h"
#include "ISceneOutlinerMode.h"
#include "SceneOutlinerPublicTypes.h"
#include "Settings/EditorStyleSettings.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

const FSceneOutlinerTreeItemType FObjectMixerEditorListRowContainer::Type(&ISceneOutlinerTreeItem::Type);

FObjectMixerEditorListRowContainer::FObjectMixerEditorListRowContainer(UObject* InPropertyOwner, FName InPropertyName, SSceneOutliner* InSceneOutliner)
	: ISceneOutlinerTreeItem(Type)
{
	PropertyOwner = InPropertyOwner;

	if (PropertyOwner.IsValid())
	{
		Property = PropertyOwner->GetClass()->FindPropertyByName(InPropertyName);
		UniqueId = HashCombineFast(GetTypeHash(InPropertyOwner), GetTypeHash(InPropertyName));
	}
}

FString FObjectMixerEditorListRowContainer::GetDisplayString() const
{
	if (!IsValid())
	{
		return TEXT("");
	}

	return Property->GetDisplayNameText().ToString();
}

class SContainerTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SContainerTreeLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FObjectMixerEditorListRowContainer& ContainerItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());

		TreeItemPtr = StaticCastSharedRef<FObjectMixerEditorListRowContainer>(ContainerItem.AsShared());

		const bool bShouldUseMiddleEllipsis = GetDefault<UEditorStyleSettings>()->bEnableMiddleEllipsis;

		ChildSlot
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
			[
				SNew(SBox)
				.WidthOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
				.HeightOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
				[
					SNew(SImage)
					.Image(this, &SContainerTreeLabel::GetIcon)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SContainerTreeLabel::GetDisplayText)
					.ToolTipText(this, &SContainerTreeLabel::GetTooltipText)
					.HighlightText(SceneOutliner.GetFilterHighlightText())
					.ColorAndOpacity(this, &SContainerTreeLabel::GetTextForegroundColor)
					.OverflowPolicy(bShouldUseMiddleEllipsis ? ETextOverflowPolicy::MiddleEllipsis : TOptional<ETextOverflowPolicy>())
				]
			]
		];
	}

private:
	FText GetDisplayText() const
	{
		if (const TSharedPtr<FObjectMixerEditorListRowContainer> TreeItem = TreeItemPtr.Pin())
		{
			return FText::FromString(TreeItem->GetDisplayString());
		}

		return FText();
	}

	FText GetTooltipText() const
	{
		if (const TSharedPtr<FObjectMixerEditorListRowContainer> TreeItem = TreeItemPtr.Pin())
		{
			return FText::FromString(TreeItem->GetDisplayString());
		}

		return FText();
	}

	const FSlateBrush* GetIcon() const
	{
		const FSlateBrush* DefaultIcon = nullptr;
		if (const TSharedPtr<FObjectMixerEditorListRowContainer> TreeItem = TreeItemPtr.Pin())
		{
			return TreeItem->PropertyIconOverride.Get(DefaultIcon);
		}
		
		return DefaultIcon;
	}

	FSlateColor GetTextForegroundColor() const
	{
		auto TreeItem = TreeItemPtr.Pin();
		if (auto BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItem))
		{
			return BaseColor.GetValue();
		}
		
		return FSlateColor::UseForeground();
	}
	
private:
	TWeakPtr<FObjectMixerEditorListRowContainer> TreeItemPtr;
};

TSharedRef<SWidget> FObjectMixerEditorListRowContainer::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SContainerTreeLabel, *this, Outliner, InRow);
}
