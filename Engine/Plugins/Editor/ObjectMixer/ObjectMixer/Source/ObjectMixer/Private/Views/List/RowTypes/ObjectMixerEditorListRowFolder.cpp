// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/RowTypes/ObjectMixerEditorListRowFolder.h"

#include "Views/List/SObjectMixerEditorList.h"

const FSceneOutlinerTreeItemType FObjectMixerEditorListRowFolder::Type(&FFolderTreeItem::Type);

void FObjectMixerEditorListRowFolder::OnVisibilityChanged(const bool bNewVisibility)
{
	RowData.OnChangeVisibility(SharedThis(this), bNewVisibility);

	if (TSharedPtr<SObjectMixerEditorList> ListView = RowData.GetListView().Pin())
	{
		ListView->EvaluateAndSetEditorVisibilityPerRow();
	}
}
