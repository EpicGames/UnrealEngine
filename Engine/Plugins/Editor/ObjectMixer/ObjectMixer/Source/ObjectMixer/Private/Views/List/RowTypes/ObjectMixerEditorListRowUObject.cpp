// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/RowTypes/ObjectMixerEditorListRowUObject.h"

#include "ClassIconFinder.h"
#include "ISceneOutlinerMode.h"
#include "Settings/EditorStyleSettings.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

const FSceneOutlinerTreeItemType FObjectMixerEditorListRowUObject::Type(&ISceneOutlinerTreeItem::Type);

FString FObjectMixerEditorListRowUObject::GetDisplayString() const
{
	if (!RowData.GetDisplayNameOverride().IsEmptyOrWhitespace())
	{
		return RowData.GetDisplayNameOverride().ToString();
	}
	
	return IsValid() && ObjectSoftPtr.IsValid() ? ObjectSoftPtr.Get()->GetName() : FString();
}

class SObjectTreeLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SObjectTreeLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FObjectMixerEditorListRowUObject& ObjectItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());

		TreeItemPtr = StaticCastSharedRef<FObjectMixerEditorListRowUObject>(ObjectItem.AsShared());
		ObjectPtr = ObjectItem.ObjectSoftPtr.Get();

		const bool bShouldUseMiddleEllipsis = GetDefault<UEditorStyleSettings>()->bEnableMiddleEllipsis;

		ChildSlot
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
			[
				SNew(SBox)
				.WidthOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
				.HeightOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
				[
					SNew(SImage)
					.Image(this, &SObjectTreeLabel::GetIcon)
					.ToolTipText(this, &SObjectTreeLabel::GetIconTooltip)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SObjectTreeLabel::GetDisplayText)
					.ToolTipText(this, &SObjectTreeLabel::GetTooltipText)
					.HighlightText(SceneOutliner.GetFilterHighlightText())
					.ColorAndOpacity(this, &SObjectTreeLabel::GetTextForegroundColor)
					.OverflowPolicy(bShouldUseMiddleEllipsis ? ETextOverflowPolicy::MiddleEllipsis : TOptional<ETextOverflowPolicy>())
				]
			]
		];
	}

private:
	FText GetDisplayText() const
	{
		if (const FSceneOutlinerTreeItemPtr TreeItem = TreeItemPtr.Pin())
		{
			return FText::FromString(TreeItem->GetDisplayString());
		}

		return FText();
	}

	FText GetTooltipText() const
	{
		if (const FSceneOutlinerTreeItemPtr TreeItem = TreeItemPtr.Pin())
		{
			return FText::FromString(TreeItem->GetDisplayString());
		}

		return FText();
	}

	const FSlateBrush* GetIcon() const
	{
		if (const UObject* Object = ObjectPtr.Get())
		{
			if (WeakSceneOutliner.IsValid())
			{
				const FSlateBrush* CachedBrush = WeakSceneOutliner.Pin()->GetCachedIconForClass(Object->GetClass()->GetFName());
				if (CachedBrush != nullptr)
				{
					return CachedBrush;
				}
				else
				{

					const FSlateBrush* FoundSlateBrush = FSlateIconFinder::FindIconForClass(Object->GetClass()).GetIcon();
					WeakSceneOutliner.Pin()->CacheIconForClass(Object->GetClass()->GetFName(), FoundSlateBrush);
					return FoundSlateBrush;
				}
			}
			else
			{
				return nullptr;
			}
		}
		else
		{
			return FSlateIconFinder::FindIconForClass(UObject::StaticClass()).GetOptionalIcon();
		}
	}
	
	FText GetIconTooltip() const
	{
		auto TreeItem = TreeItemPtr.Pin();
		if (!TreeItem.IsValid())
		{
			return FText();
		}

		FText ToolTipText;
		if (UObject* Object = ObjectPtr.Get())
		{
			ToolTipText = FText::FromString(Object->GetClass()->GetName());
		}

		return ToolTipText;
	}

	FSlateColor GetTextForegroundColor() const
	{
		auto TreeItem = TreeItemPtr.Pin();
		if (auto BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItem))
		{
			return BaseColor.GetValue();
		}
		
		return FSlateColor::UseForeground();
	}
	
private:
	TWeakPtr<FObjectMixerEditorListRowUObject> TreeItemPtr;
	TWeakObjectPtr<UObject> ObjectPtr;
};

TSharedRef<SWidget> FObjectMixerEditorListRowUObject::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SObjectTreeLabel, *this, Outliner, InRow);
}
