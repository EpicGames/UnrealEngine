// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/List/SObjectMixerEditorList.h"

#include "ObjectMixerEditorModule.h"
#include "ObjectMixerEditorSerializedData.h"
#include "ObjectMixerEditorSettings.h"
#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"
#include "Views/List/ObjectMixerEditorList.h"
#include "Views/List/ObjectMixerUtils.h"
#include "Views/List/Modes/ObjectMixerOutlinerMode.h"
#include "Views/List/Modes/OutlinerColumns/ObjectMixerOutlinerColumns.h"
#include "Views/List/RowTypes/ObjectMixerEditorListRowActor.h"
#include "Views/Widgets/SCollectionSelectionButton.h"
#include "Views/Widgets/SObjectMixerPlacementAssetMenuEntry.h"

#include "ActorDescTreeItem.h"
#include "ActorFolderTreeItem.h"
#include "ComponentTreeItem.h"
#include "Editor.h"
#include "EditorActorFolders.h"
#include "IPlacementModeModule.h"
#include "LevelEditorSequencerIntegration.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerTextInfoColumn.h"
#include "SPositiveActionButton.h"

#include "Algo/Find.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Views/STreeView.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleEngineSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SObjectMixerEditorList)

#define LOCTEXT_NAMESPACE "SObjectMixerEditorList"

const FName SObjectMixerEditorList::ItemNameColumnName(TEXT("Builtin_Name"));
const FName SObjectMixerEditorList::EditorVisibilityColumnName(TEXT("Builtin_EditorVisibility"));
const FName SObjectMixerEditorList::EditorVisibilitySoloColumnName(TEXT("Builtin_EditorVisibilitySolo"));

const FName SequencerSpawnableColumnName("Spawnable");
const FName SequencerInfoColumnName("Sequence");

void CloseAllMenus()
{
	FSlateApplication::Get().DismissAllMenus();
}

void SObjectMixerEditorList::Construct(const FArguments& InArgs, TSharedRef<FObjectMixerEditorList> ListModel)
{
	ListModelPtr = ListModel;

	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.bShowSearchBox = true;
	InitOptions.bFocusSearchBoxWhenOpened = true;
	InitOptions.bShowCreateNewFolder = true;
	InitOptions.bShowParentTree = true;
	InitOptions.bShowTransient = ListModel->ShouldShowTransientObjects();
	InitOptions.OutlinerIdentifier = ListModel->GetModuleName();
	
	// Mode
	InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([this](SSceneOutliner* InOutliner)
	{		
		return new FObjectMixerOutlinerMode(
			FObjectMixerOutlinerModeParams(
				InOutliner, nullptr, false,
				false, true, true
			),
			ListModelPtr.Pin().ToSharedRef()
		);
	});

	// Header Row
	InitOptions.bShowHeaderRow = true;
	InitOptions.bCanSelectGeneratedColumns = false;
	 
	SetupColumns(InitOptions);
	SSceneOutliner::Construct(InArgs, InitOptions);

	InsertCollectionSelector();

	if (UObjectMixerEditorSerializedData* SerializedData = GetMutableDefault<UObjectMixerEditorSerializedData>())
	{
		SerializedData->OnObjectMixerCollectionMapChanged.AddSP(this, &SObjectMixerEditorList::RebuildCollectionSelector);
	}
	
	RebuildCollectionSelector();

	SetSingleCollectionSelection();
}

SObjectMixerEditorList::~SObjectMixerEditorList()
{	
	HeaderRowContextMenuWidget.Reset();
}

FReply SObjectMixerEditorList::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Check to see if any actions can be processed
	// If we are in debug mode do not process commands
	if (FSlateApplication::Get().IsNormalExecution())
	{
		if (GetListModelPtr().Pin()->ObjectMixerElementEditCommands->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	}
	
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

bool SObjectMixerEditorList::CanCreateFolder() const
{
	return GetSelectedTreeViewItemCount() > 0;
}

TSharedRef<SWidget> SObjectMixerEditorList::OnGenerateAddObjectButtonMenu() const
{
	TSet<UClass*> ClassesToPlace;
	for (const TObjectPtr<UObjectMixerObjectFilter>& Instance : ListModelPtr.Pin()->GetObjectFilterInstances())
	{
		const TSet<TSubclassOf<AActor>> SubclassesOfActor = Instance->GetObjectClassesToPlace();
		if (SubclassesOfActor.Num() > 0)
		{
			ClassesToPlace.Append(
				Instance->GetParentAndChildClassesFromSpecifiedClasses(
					SubclassesOfActor,
					Instance->GetObjectMixerPlacementClassInclusionOptions()
				)
			);
		}
	}

	if (ClassesToPlace.Num() > 0)
	{
		FMenuBuilder AddObjectButtonMenuBuilder = FMenuBuilder(true, nullptr);

		for (UClass* Class : ClassesToPlace)
		{
			TSharedPtr<FPlaceableItem> Item = MakeShared<FPlaceableItem>(GEditor->FindActorFactoryForActorClass(Class), FAssetData(Class));
			AddObjectButtonMenuBuilder.AddWidget(SNew(SObjectMixerPlacementAssetMenuEntry, Item), FText::GetEmpty());
		}

		return AddObjectButtonMenuBuilder.MakeWidget();
	}

	return
		SNew(SBox)
		.Padding(5)
		[
			SNew(STextBlock)
				.Text(LOCTEXT("NoPlaceableActorsDefinedWarning", "Please define some placeable actors in the\nfilter class by overriding ForceGetObjectClassesToPlace."))
				.Font(FAppStyle::Get().GetFontStyle("NormalFontItalic"))
		]
	;
}

void SObjectMixerEditorList::SetSingleCollectionSelection(const FName& CollectionToEnableName)
{
	// Disable all collection filters except CollectionToEnableName
	GetListModelPtr().Pin()->SetSelectedCollections({CollectionToEnableName});
}


EObjectMixerTreeViewMode SObjectMixerEditorList::GetTreeViewMode()
{
	return ListModelPtr.Pin()->GetTreeViewMode();
}

void SObjectMixerEditorList::SetTreeViewMode(EObjectMixerTreeViewMode InViewMode)
{
	ListModelPtr.Pin()->SetTreeViewMode(InViewMode);
}

const TSet<FName>& SObjectMixerEditorList::GetSelectedCollections()
{
	return GetListModelPtr().Pin()->GetSelectedCollections();
}

bool SObjectMixerEditorList::IsCollectionSelected(const FName& CollectionName)
{
	return GetListModelPtr().Pin()->IsCollectionSelected(CollectionName);
}

void SObjectMixerEditorList::RebuildCollectionSelector()
{
	// Make user collections

	CollectionSelectorBox->ClearChildren();
	CollectionSelectorBox->SetVisibility(EVisibility::Collapsed);

	auto CreateCollectionFilterAndAddToCollectionSelector =
		[this](const FName& InCollectionName) 
	{
		CollectionSelectorBox->AddSlot()
		[
			SNew(SCollectionSelectionButton, SharedThis(this), InCollectionName)
		];
	};

	TArray<FName> AllCollectionNames = ListModelPtr.Pin()->GetAllCollectionNames();
	
	// If no collections, rather than show "All" just keep the box hidden
	// Otherwise make "All" collection widget
	if (AllCollectionNames.Num())
	{
		CreateCollectionFilterAndAddToCollectionSelector(UObjectMixerEditorSerializedData::AllCollectionName);
	}
	
	for (const FName& CollectionName : AllCollectionNames)
	{
		// Create widgets for each name
		CreateCollectionFilterAndAddToCollectionSelector(CollectionName);
	}

	CollectionSelectorBox->SetVisibility(EVisibility::Visible);
}

bool SObjectMixerEditorList::RequestRemoveCollection(const FName& CollectionName)
{
	if (ListModelPtr.Pin()->RequestRemoveCollection(CollectionName))
	{
		OnCollectionCheckedStateChanged(true, UObjectMixerEditorSerializedData::AllCollectionName);

		RebuildCollectionSelector();

		return true;
	}

	return false;
}

bool SObjectMixerEditorList::RequestDuplicateCollection(const FName& CollectionToDuplicateName, FName& DesiredDuplicateName) const
{
	return ListModelPtr.Pin()->RequestDuplicateCollection(CollectionToDuplicateName, DesiredDuplicateName);
}

bool SObjectMixerEditorList::RequestRenameCollection(
	const FName& CollectionNameToRename,
	const FName& NewCollectionName)
{
	return ListModelPtr.Pin()->RequestRenameCollection(CollectionNameToRename, NewCollectionName);
}

bool SObjectMixerEditorList::DoesCollectionExist(const FName& CollectionName) const
{
	return ListModelPtr.Pin()->DoesCollectionExist(CollectionName);
}

void SObjectMixerEditorList::OnCollectionCheckedStateChanged(bool bShouldBeChecked, FName CollectionName)
{
	GetListModelPtr().Pin()->OnPreFilterChangeDelegate.Broadcast();

	const bool bIsAllCollection = CollectionName.IsEqual(UObjectMixerEditorSerializedData::AllCollectionName);

	if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
	{
		if (bIsAllCollection)
		{
			// Can't disable All
			return;
		}
		
		GetListModelPtr().Pin()->SetCollectionSelected(CollectionName, bShouldBeChecked);

		if (!bShouldBeChecked && GetSelectedCollections().Num() == 0)
		{
			// Reset to all
			SetSingleCollectionSelection();
		}
	}
	else
	{
		if (!bShouldBeChecked && bIsAllCollection)
		{
			// Reset to all
			SetSingleCollectionSelection();
		}
		
		// Set just this filter active
		SetSingleCollectionSelection(CollectionName);
	}

	GetListModelPtr().Pin()->OnPostFilterChangeDelegate.Broadcast();
}

ECheckBoxState SObjectMixerEditorList::GetCollectionCheckedState(FName CollectionName) const
{
	return GetListModelPtr().Pin()->IsCollectionSelected(CollectionName) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FObjectMixerOutlinerMode* SObjectMixerEditorList::GetCastedMode() const
{
	return StaticCast<FObjectMixerOutlinerMode*>(Mode);
}

UWorld* SObjectMixerEditorList::GetWorld() const
{
	if (FObjectMixerOutlinerMode* CastedMode = GetCastedMode())
	{
		if (UWorld* UserChosenWorld = CastedMode->GetRepresentingWorld())
		{
			return UserChosenWorld;
		}
	}
	
	return FObjectMixerEditorModule::Get().GetWorld();
}

void SObjectMixerEditorList::RebuildList()
{
	bIsRebuildRequested = false;
	
	Mode->Rebuild();
	RebuildCollectionSelector();
}

void SObjectMixerEditorList::RefreshList()
{
	GetTreeView()->RequestListRefresh();
}

void SObjectMixerEditorList::RequestRebuildList(const FString& InItemToScrollTo)
{
	bIsRebuildRequested = true;
}

void SObjectMixerEditorList::InsertCollectionSelector()
{
	if (const TSharedPtr<SHorizontalBox> PinnedToolbar = ToolbarPtr.Pin())
	{
		if (TSharedPtr<SVerticalBox> VerticalBoxParent = StaticCastSharedPtr<SVerticalBox>(PinnedToolbar->GetParentWidget()))
		{
			int32 InsertAt = 2; // Best guess in case we can't find it procedurally

			for (int32 SlotItr = 0; SlotItr < VerticalBoxParent->NumSlots(); SlotItr++)
			{
				if (VerticalBoxParent->IsValidSlotIndex(SlotItr))
				{
					if (VerticalBoxParent->GetSlot(SlotItr).GetWidget() == PinnedToolbar)
					{
						InsertAt = SlotItr + 1;
					}
				}
			}
			
			VerticalBoxParent->InsertSlot(InsertAt)
			.Padding(8, 2, 8, 7)
			.AutoHeight()
			[
				SAssignNew(CollectionSelectorBox, SWrapBox)
				.UseAllottedSize(true)
				.InnerSlotPadding(FVector2D(4,4))
			];
		}
	}
}

void SObjectMixerEditorList::OnRenameCommand()
{
	
}

void SObjectMixerEditorList::AddToPendingPropertyPropagations(
	const FObjectMixerEditorListRowData::FPropertyPropagationInfo& InPropagationInfo)
{
	PendingPropertyPropagations.Add(InPropagationInfo);
}

TArray<TSharedPtr<ISceneOutlinerTreeItem>> SObjectMixerEditorList::GetSelectedTreeViewItems() const
{
	if (const TSharedPtr<SSceneOutlinerTreeView> Tree = GetTreeView())
	{
		return Tree->GetSelectedItems();
	}

	return {};
}

int32 SObjectMixerEditorList::GetSelectedTreeViewItemCount() const
{
	if (const TSharedPtr<SSceneOutlinerTreeView> Tree = GetTreeView())
	{
		return Tree->GetSelectedItems().Num();
	}

	return INDEX_NONE;
}

void SObjectMixerEditorList::SetTreeViewItemSelected(TSharedRef<ISceneOutlinerTreeItem> Item, const bool bNewSelected)
{
	if (const TSharedPtr<SSceneOutlinerTreeView> Tree = GetTreeView())
	{
		Tree->SetItemSelection(Item, bNewSelected);
	}
}

bool SObjectMixerEditorList::IsTreeViewItemSelected(TSharedPtr<ISceneOutlinerTreeItem> Item)
{
	if (const TSharedPtr<SSceneOutlinerTreeView> Tree = GetTreeView())
	{
		return Tree->IsItemSelected(Item);
	}
	
	return false;
}

TSet<TSharedPtr<ISceneOutlinerTreeItem>> SObjectMixerEditorList::GetSoloRows() const
{
	TSet<TSharedPtr<ISceneOutlinerTreeItem>> SoloRows;
	
	using LambdaType = void(*)(const TSharedPtr<ISceneOutlinerTreeItem>&, TSet<TSharedPtr<ISceneOutlinerTreeItem>>&);
	static LambdaType GetSoloRowsRecursively =
		[](const TSharedPtr<ISceneOutlinerTreeItem>& InRow, TSet<TSharedPtr<ISceneOutlinerTreeItem>>& SoloRows)
	{
		if (FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(InRow))
		{
			if (RowData->GetRowSoloState())
			{
				SoloRows.Add(InRow);
			}
		}
			
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : InRow->GetChildren())
		{
			if (Child.IsValid())
			{
				GetSoloRowsRecursively(Child.Pin(), SoloRows);
			}
		}
	};

	for (const TSharedPtr<ISceneOutlinerTreeItem>& RootRow : GetTreeRootItems())
	{
		GetSoloRowsRecursively(RootRow, SoloRows);
	}

	return SoloRows;
}

TSet<TSharedPtr<ISceneOutlinerTreeItem>> SObjectMixerEditorList::GetTreeRootItems() const
{
	if (GetTree().GetRootItems().Num())
	{
		return {GetTree().GetRootItems()[0]};
	}

	return {};
}

TSet<TWeakPtr<ISceneOutlinerTreeItem>> SObjectMixerEditorList::GetWeakTreeRootItems() const
{
	if (GetTree().GetRootItems().Num())
	{
		return {GetTree().GetRootItems()[0]};
	}

	return {};
}

void SObjectMixerEditorList::ClearSoloRows()
{
	using LambdaType = void(*)(const TSharedPtr<ISceneOutlinerTreeItem>&);
	static LambdaType ClearSoloRowsRecursively = [](const TSharedPtr<ISceneOutlinerTreeItem>& InRow)
	{
		if (FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(InRow))
		{
			RowData->SetRowSoloState(false);
		}
			
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : InRow->GetChildren())
		{
			if (Child.IsValid())
			{
				ClearSoloRowsRecursively(Child.Pin());
			}
		}
	};

	for (const TSharedPtr<ISceneOutlinerTreeItem>& RootItem : GetTreeRootItems())
	{
		ClearSoloRowsRecursively(RootItem);
	}
}

bool SObjectMixerEditorList::IsListInSoloState() const
{
	using LambdaType = bool(*)(const TSharedPtr<ISceneOutlinerTreeItem>&);
	static LambdaType GetSoloRowsRecursively = [](const TSharedPtr<ISceneOutlinerTreeItem>& InRow)
	{
		if (FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(InRow))
		{
			if (RowData->GetRowSoloState())
			{
				return true;
			}
		}
			
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : InRow->GetChildren())
		{
			if (Child.IsValid())
			{
				if (GetSoloRowsRecursively(Child.Pin()))
				{
					return true;
				}
			}
		}

		return false;
	};

	for (const TSharedPtr<ISceneOutlinerTreeItem>& RootItem : GetTreeRootItems())
	{
		if (GetSoloRowsRecursively(RootItem))
		{
			return true;
		}
	}

	return false;
}

void SObjectMixerEditorList::EvaluateAndSetEditorVisibilityPerRow()
{
	using LambdaType = void(*)(const TSharedPtr<ISceneOutlinerTreeItem>&, const bool);
	static LambdaType EvaluateEditorVisibilityRecursively =
		[](const TSharedPtr<ISceneOutlinerTreeItem>& InRow, const bool bIsListInSoloState)
	{
		if (const FObjectMixerEditorListRowActor* ActorRow = FObjectMixerUtils::AsActorRow(InRow))
		{
			bool bShouldBeHidden = ActorRow->RowData.IsUserSetHiddenInEditor();

			if (bIsListInSoloState)
			{
				bShouldBeHidden = !ActorRow->RowData.GetRowSoloState();
			}

			// Only create a transaction in the case of a change
			if (bShouldBeHidden != ActorRow->Actor->IsTemporarilyHiddenInEditor())
			{
				// Save the actor to the transaction buffer to support undo/redo, but do
				// not call Modify, as we do not want to dirty the actor's package and
				// we're only editing temporary, transient values
				SaveToTransactionBuffer(ActorRow->Actor.Get(), false);
				ActorRow->Actor->SetIsTemporarilyHiddenInEditor(bShouldBeHidden);
			}
		}
			
		for (const TWeakPtr<ISceneOutlinerTreeItem>& Child : InRow->GetChildren())
		{
			if (Child.IsValid())
			{
				EvaluateEditorVisibilityRecursively(Child.Pin(), bIsListInSoloState);
			}
		}
	};

	const bool bIsListInSoloState = IsListInSoloState();

	for (const TSharedPtr<ISceneOutlinerTreeItem>& RootItem : GetTreeRootItems())
	{
		EvaluateEditorVisibilityRecursively(RootItem, bIsListInSoloState);
	}
}

bool SObjectMixerEditorList::IsTreeViewItemExpanded(const TSharedPtr<ISceneOutlinerTreeItem>& Row) const
{
	return GetTree().IsItemExpanded(Row);
}

void SObjectMixerEditorList::SetTreeViewItemExpanded(const TSharedPtr<ISceneOutlinerTreeItem>& RowToExpand, const bool bNewExpansion) const
{
	GetTreeView()->SetItemExpansion(RowToExpand, bNewExpansion);
}

FObjectMixerSceneOutlinerColumnInfo* SObjectMixerEditorList::GetColumnInfoByPropertyName(const FName& InPropertyName)
{
	return Algo::FindByPredicate(HeaderColumnInfos,
		[InPropertyName] (const FObjectMixerSceneOutlinerColumnInfo& ColumnInfo)
		{
			return ColumnInfo.PropertyName.IsEqual(InPropertyName);
		});
}

void SObjectMixerEditorList::RestoreDefaultPropertyColumns()
{
	for (const FObjectMixerSceneOutlinerColumnInfo& ColumnInfo : HeaderColumnInfos)
	{
		SetColumnVisibility(ColumnInfo.ColumnID, ColumnInfo.bIsDesiredToBeShownByDefault);
	}
}

void SObjectMixerEditorList::CustomAddToToolbar(TSharedPtr<SHorizontalBox> Toolbar)
{
	ToolbarPtr = Toolbar;

	// "Add object" button
	Toolbar->InsertSlot(0)
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.AutoWidth()
	.Padding(FMargin(0, 4, 8, 4))
	[
		SNew(SPositiveActionButton)
		.Text(LOCTEXT("AddObject", "Add"))
		.OnGetMenuContent(FOnGetContent::CreateRaw(this, &SObjectMixerEditorList::OnGenerateAddObjectButtonMenu))
	];

	// Sync selection toggle
	Toolbar->AddSlot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.AutoWidth()
	.Padding(8.f, 1.f, 0.f, 1.f)
	[
		SNew(SCheckBox )
		.Padding(4.f)
		.ToolTipText(LOCTEXT("SyncSelectionButton_Tooltip", "Sync Selection\nIf enabled, clicking an item in the mixer list will also select the item in the Scene Outliner.\nAlt + Click to select items in mixer without selecting the item in the Scene outliner.\nIf disabled, selections will not sync unless Alt is held. Effectively, this is the opposite behavior."))
		.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
		.IsChecked_Lambda([]()
		{
			if (const UObjectMixerEditorSettings* Settings = GetDefault<UObjectMixerEditorSettings>())
			{
				return Settings->bSyncSelection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
			
			return ECheckBoxState::Undetermined; 
		})
		.OnCheckStateChanged_Lambda([](ECheckBoxState InNewState)
		{
			if (UObjectMixerEditorSettings* Settings = GetMutableDefault<UObjectMixerEditorSettings>())
			{
				Settings->bSyncSelection = InNewState == ECheckBoxState::Checked ? true : false;
			}
		})
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image( FAppStyle::Get().GetBrush("FoliageEditMode.SelectAll") )
		]
	];
}

void SObjectMixerEditorList::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Early out so we have widgets to act upon next frame
	SSceneOutliner::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	if (bIsRebuildRequested)
	{
		RebuildList();
	}

	if (PendingPropertyPropagations.Num() > 0)
	{
		PropagatePropertyChangesToSelectedRows();
		if (PendingPropertyPropagations.Num() == 0)
		{
			RequestRebuildList();
		}
	}
}

TSharedRef<SWidget> SObjectMixerEditorList::GenerateHeaderRowContextMenu()
{
	TSharedPtr<FObjectMixerEditorList> PinnedListModel = ListModelPtr.Pin(); 
	check(PinnedListModel);

	if (const UObjectMixerObjectFilter* Filter = PinnedListModel->GetMainObjectFilterInstance())
	{
		if (!Filter->ShouldAllowColumnCustomizationByUser())
		{
			return SNullWidget::NullWidget;
		}
	}

	if (!HeaderRowContextMenuWidget)
	{
		FMenuBuilder MenuBuilder(false, nullptr);
	
		MenuBuilder.AddSearchWidget();

		MenuBuilder.BeginSection(TEXT("Actions"), LOCTEXT("ContextMenuActions", "Actions"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RestoreDefaultPropertyColumns", "Restore Default Columns"),
				LOCTEXT("RestoreDefaultPropertyColumnsTooltip", "Restore property columns to built-in columns and those as defined in GetColumnsToShowByDefault() in your object filters."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this]()
					{
						CloseAllMenus();
						RestoreDefaultPropertyColumns();
					})
				)
			);
		}
		MenuBuilder.EndSection();

		FName LastPropertyCategoryName = NAME_None;

		for (const FObjectMixerSceneOutlinerColumnInfo& ColumnInfo : HeaderColumnInfos)
		{
			const FName& PropertyCategoryName = ColumnInfo.PropertyCategoryName;

			if (!PropertyCategoryName.IsEqual(LastPropertyCategoryName))
			{
				LastPropertyCategoryName = PropertyCategoryName;
			
				MenuBuilder.EndSection();
				MenuBuilder.BeginSection(LastPropertyCategoryName, FText::FromName(LastPropertyCategoryName));
			}
		
			const FName& ColumnID = ColumnInfo.ColumnID;
		
			const FText Tooltip = ColumnInfo.PropertyRef ?
				ColumnInfo.PropertyRef->GetToolTipText() : ColumnInfo.PropertyDisplayText;

			const bool bCanSelectColumn = ColumnInfo.bCanBeHidden;

			const FName Hook = ColumnInfo.PropertyType == EListViewColumnType::BuiltIn ? "Builtin" : "GeneratedProperties";
		
			MenuBuilder.AddMenuEntry(
				ColumnInfo.PropertyDisplayText,
				Tooltip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, ColumnID]()
					{
						TSharedPtr<FObjectMixerEditorList> PinnedListModel = ListModelPtr.Pin();
						TSharedPtr<SHeaderRow> PinnedHeaderRow = GetTree().GetHeaderRow();
						check(PinnedHeaderRow);

						const bool bNewColumnEnabled = !PinnedHeaderRow->IsColumnVisible(ColumnID);
					
						PinnedHeaderRow->SetShowGeneratedColumn(ColumnID, bNewColumnEnabled);
					}),
					FCanExecuteAction::CreateLambda([bCanSelectColumn](){return bCanSelectColumn;}),
					FIsActionChecked::CreateLambda([this, ColumnID]()
					{
						return GetTree().GetHeaderRow()->IsColumnVisible(ColumnID);
					})
				),
				Hook,
				EUserInterfaceActionType::ToggleButton
			);
		}

		HeaderRowContextMenuWidget = MenuBuilder.MakeWidget();
	}

	return HeaderRowContextMenuWidget.ToSharedRef();
}

bool SObjectMixerEditorList::AddUniquePropertyColumnInfo(
	FProperty* Property,
	const bool bForceIncludeProperty,
	const TSet<FName>& PropertySkipList)
{
	if (!ensureAlwaysMsgf(Property, TEXT("%hs: Invalid property passed in. Please ensure only valid properties are passed to this function."), __FUNCTION__))
	{
		return false;
	}

	bool bShouldIncludeProperty = bForceIncludeProperty;

	if (!bShouldIncludeProperty)
	{
		const bool bIsPropertyBlueprintEditable = (Property->GetPropertyFlags() & CPF_Edit) != 0;

		// We don't have a proper way to display these yet
		const bool bDoesPropertyHaveSupportedClass =
			!Property->IsA(FMapProperty::StaticClass()) &&
			!Property->IsA(FArrayProperty::StaticClass()) &&
			!Property->IsA(FSetProperty::StaticClass()) &&
			!Property->IsA(FStructProperty::StaticClass());

		bShouldIncludeProperty = bIsPropertyBlueprintEditable && bDoesPropertyHaveSupportedClass;
	}

	if (bShouldIncludeProperty)
	{
		const bool bIsPropertyExplicitlySkipped =
		   PropertySkipList.Num() && PropertySkipList.Contains(Property->GetFName());

		bShouldIncludeProperty = !bIsPropertyExplicitlySkipped;
	}
	
	if (bShouldIncludeProperty)
	{
		const FName PropertyName = Property->GetFName();
	
		// Ensure no duplicate properties
		if (!Algo::FindByPredicate(HeaderColumnInfos,
				[&PropertyName] (const FObjectMixerSceneOutlinerColumnInfo& ListViewColumn)
				{
					return ListViewColumn.PropertyName.IsEqual(PropertyName);
				})
			)
		{
			FString PropertyCategoryName = "Generated Properties";
			if (const FString* CategoryMeta = Property->FindMetaData("Category"))
			{
				PropertyCategoryName = *CategoryMeta;
			}
			
			HeaderColumnInfos.Add(
				{
					Property, PropertyName, FObjectMixerOutlinerPropertyColumn::GetID(Property),
					Property->GetDisplayNameText(),
					EListViewColumnType::PropertyGenerated,
					*PropertyCategoryName,
					true,
					bForceIncludeProperty ? true : GetListModelPtr().Pin()->ColumnsToShowByDefaultCache.Contains(PropertyName)
				}
			);

			return true;
		}
	}

	return false;
}

// The "Level" column should be named "Package Short Name" in wp enabled levels
FText SObjectMixerEditorList::GetLevelColumnName() const
{
	const UWorld* WorldPtr = GetWorld();
	if (WorldPtr && WorldPtr->IsPartitionedWorld())
	{
		return FSceneOutlinerBuiltInColumnTypes::PackageShortName_Localized();
	}

	return FSceneOutlinerBuiltInColumnTypes::Level_Localized();
}

void SObjectMixerEditorList::CreateActorTextInfoColumns(FSceneOutlinerInitializationOptions& OutInitOptions)
{
	FGetTextForItem LayerInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		FString Result;
		
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			AActor* Actor = ActorItem->Actor.Get();
			if (!Actor)
			{
				return FString();
			}

			for (const auto& Layer : Actor->Layers)
			{
				if (Result.Len())
				{
					Result += TEXT(", ");
				}

				Result += Layer.ToString();
			}
		}

		return Result;
	});

	FGetTextForItem DataLayerInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		TStringBuilder<128> Builder;
		TSet<FString> DataLayerShortNames;

		auto BuildDataLayers = [&Builder, &DataLayerShortNames](const auto& DataLayerInstances, bool bPartOfOtherLevel)
		{
			for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
			{
				bool bIsAlreadyInSet = false;
				DataLayerShortNames.Add(DataLayerInstance->GetDataLayerShortName(), &bIsAlreadyInSet);
				if (!bIsAlreadyInSet)
				{
					if (Builder.Len())
					{
						Builder += TEXT(", ");
					}
					// Put a '*' in front of DataLayers that are not part of of the main world
					if (bPartOfOtherLevel)
					{
						Builder += "*";
					}
					Builder += DataLayerInstance->GetDataLayerShortName();
				}
			}
		};

		auto BuildDataLayersWithContext = [BuildDataLayers](const ISceneOutlinerTreeItem& Item, bool bUseLevelContext)
		{
			if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
			{
				if (AActor* Actor = ActorItem->Actor.Get())
				{
					BuildDataLayers(bUseLevelContext ? Actor->GetDataLayerInstancesForLevel() : Actor->GetDataLayerInstances(), bUseLevelContext);
				}
			}
			else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
			{
				if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle; ActorDescInstance && !ActorDescInstance->GetDataLayerInstanceNames().IsEmpty())
				{
					if (const UActorDescContainerInstance* ActorDescContainerInstance = ActorDescInstance->GetContainerInstance())
					{
						const UWorld* OwningWorld = ActorDescContainerInstance->GetOuterWorldPartition()->GetWorld();
						if (const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(OwningWorld))
						{
							TSet<const UDataLayerInstance*> DataLayerInstances;
							DataLayerInstances.Append(DataLayerManager->GetDataLayerInstances(ActorDescInstance->GetDataLayerInstanceNames().ToArray()));
							if (ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(OwningWorld))
							{
								UWorld* OuterWorld = ActorDescContainerInstance->GetTypedOuter<UWorld>();
								// Add parent container Data Layer Instances
								AActor* CurrentActor = OuterWorld ? Cast<AActor>(LevelInstanceSubsystem->GetOwningLevelInstance(OuterWorld->PersistentLevel)) : nullptr;
								while (CurrentActor)
								{
									DataLayerInstances.Append(bUseLevelContext ? CurrentActor->GetDataLayerInstancesForLevel() : CurrentActor->GetDataLayerInstances());
									CurrentActor = Cast<AActor>(LevelInstanceSubsystem->GetParentLevelInstance(CurrentActor));
								};
							}
							BuildDataLayers(DataLayerInstances, bUseLevelContext);
						}
					}
				}
			}
		};

		// List Actor's DataLayers part of the owning world, then those only part of the actor level
		BuildDataLayersWithContext(Item, false);
		BuildDataLayersWithContext(Item, true);

		return Builder.ToString();
	});

	FGetTextForItem ContentBundleInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item)->FString
	{	
		UContentBundleEngineSubsystem* ContentBundleEngineSubsystem = UContentBundleEngineSubsystem::Get();
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				if (const UContentBundleDescriptor * Descriptor = ContentBundleEngineSubsystem->GetContentBundleDescriptor(Actor->GetContentBundleGuid()))
				{
					return Descriptor->GetDisplayName();
				}
			}
		}
		else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				if (const UContentBundleDescriptor* Descriptor = ContentBundleEngineSubsystem->GetContentBundleDescriptor(ActorDescInstance->GetContentBundleGuid()))
				{
					return Descriptor->GetDisplayName();
				}
			}
		}

		return TEXT("");
	});

	FGetTextForItem SubPackageInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				if (const ILevelInstanceInterface* ActorAsLevelInstance = Cast<ILevelInstanceInterface>(Actor))
				{
					return ActorAsLevelInstance->GetWorldAssetPackage();
				}
			}
		}
		else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				if (FName LevelPackage = ActorDescInstance->GetChildContainerPackage(); !LevelPackage.IsNone())
				{
					return ActorDescInstance->GetChildContainerPackage().ToString();
				}
			}
		}

		return FString();
	});

	FGetTextForItem SocketInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			AActor* Actor = ActorItem->Actor.Get();

			if (Actor)
			{
				return Actor->GetAttachParentSocketName().ToString();
			}
		}
		
		return FString();
	});

	FGetTextForItem InternalNameInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				return Actor->GetFName().ToString();
			}
		}
		else if (const FComponentTreeItem* ComponentItem = Item.CastTo<FComponentTreeItem>())
		{
			if (UActorComponent* Component = ComponentItem->Component.Get())
			{
				return Component->GetFName().ToString();
			}
		}
		else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				return ActorDescInstance->GetActorName().ToString();
			}
		}
		else if (const FActorFolderTreeItem* ActorFolderItem = Item.CastTo<FActorFolderTreeItem>())
		{
			if (const UActorFolder* ActorFolder = Cast<UActorFolder>(ActorFolderItem->GetActorFolder()))
			{
				return ActorFolder->GetFName().ToString();
			}
		}

		return FString();
	});

	FGetTextForItem LevelInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				return FPackageName::GetShortName(Actor->GetPackage()->GetName());
			}
		}
		else if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				return FPackageName::GetShortName(ActorDescInstance->GetActorPackage());
			}
		}
		else if (const FActorFolderTreeItem* ActorFolderItem = Item.CastTo<FActorFolderTreeItem>())
		{
			if (const UActorFolder* ActorFolder = Cast<UActorFolder>(ActorFolderItem->GetActorFolder()))
			{
				return FPackageName::GetShortName(ActorFolder->GetPackage()->GetName());
			}
		}

		return FString();
	});

	FGetTextForItem UncachedLightsInfoText = FGetTextForItem::CreateLambda([](const ISceneOutlinerTreeItem& Item) -> FString
	{
		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			AActor* Actor = ActorItem->Actor.Get();
			if (Actor)
			{
				return FString::Printf(TEXT("%7d"), Actor->GetNumUncachedStaticLightingInteractions());
			}
		}
		return FString();
	});

	TSharedPtr<FObjectMixerEditorList> PinnedListModel = ListModelPtr.Pin();

	auto AddTextInfoColumn = [this, PinnedListModel, &OutInitOptions](FName ColumnID, TAttribute<FText> ColumnName, FGetTextForItem ColumnInfo)
	{
		bool bShouldShowColumn = false;
		
		OutInitOptions.ColumnMap.Add(
			ColumnID,
			FSceneOutlinerColumnInfo(
				bShouldShowColumn ? ESceneOutlinerColumnVisibility::Visible : ESceneOutlinerColumnVisibility::Invisible,
				20,
				FCreateSceneOutlinerColumn::CreateLambda([ColumnID, ColumnName, ColumnInfo](ISceneOutliner& Outliner)
					{
						return TSharedRef<FObjectMixerOutlinerTextInfoColumn>(
							MakeShared<FObjectMixerOutlinerTextInfoColumn>(Outliner, ColumnID, ColumnInfo, FText::GetEmpty()));
					}
				),
				true,
				TOptional<float>(),
				ColumnName,
				EHeaderComboVisibility::Never,
				FOnGetContent::CreateRaw(this, &SObjectMixerEditorList::GenerateHeaderRowContextMenu)
			)
		);

		HeaderColumnInfos.Add(
			{
				nullptr, ColumnID, ColumnID,
				ColumnName.Get(FText::GetEmpty()),
				EListViewColumnType::BuiltIn,
				TEXT("Built-In"),
				true, bShouldShowColumn
			}
		);
	};

	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::Level(), TAttribute<FText>::CreateSP(this, &SObjectMixerEditorList::GetLevelColumnName), LevelInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::Layer(), FSceneOutlinerBuiltInColumnTypes::Layer_Localized(), LayerInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::DataLayer(), FSceneOutlinerBuiltInColumnTypes::DataLayer_Localized(), DataLayerInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::ContentBundle(), FSceneOutlinerBuiltInColumnTypes::ContentBundle_Localized(), ContentBundleInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::SubPackage(), FSceneOutlinerBuiltInColumnTypes::SubPackage_Localized(), SubPackageInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::Socket(), FSceneOutlinerBuiltInColumnTypes::Socket_Localized(), SocketInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::IDName(), FSceneOutlinerBuiltInColumnTypes::IDName_Localized(), InternalNameInfoText);
	AddTextInfoColumn(FSceneOutlinerBuiltInColumnTypes::UncachedLights(), FSceneOutlinerBuiltInColumnTypes::UncachedLights_Localized(), UncachedLightsInfoText);
}

void SObjectMixerEditorList::SetupColumns(FSceneOutlinerInitializationOptions& OutInitOptions)
{
	TSharedPtr<FObjectMixerEditorList> PinnedListModel = ListModelPtr.Pin();
	check(PinnedListModel);
	
	HeaderColumnInfos.Empty(HeaderColumnInfos.Num());

	// Add Built-In Columns to Header
	{
		auto AddBuiltInColumn = [&](
			ESceneOutlinerColumnVisibility InDefaultVisibility, int32 InPriorityIndex, const FName ColumnID,
			const FText DisplayText, const bool bCanBeHidden, FCreateSceneOutlinerColumn CreationDelegate = FCreateSceneOutlinerColumn(),
			TOptional<float> FillSize = TOptional<float>())
		{			
			OutInitOptions.ColumnMap.Add(ColumnID, FSceneOutlinerColumnInfo(
				InDefaultVisibility,
				InPriorityIndex, CreationDelegate, bCanBeHidden,
				FillSize, DisplayText, EHeaderComboVisibility::Never,
				FOnGetContent::CreateRaw(this, &SObjectMixerEditorList::GenerateHeaderRowContextMenu)));

			HeaderColumnInfos.Add(
				{
					nullptr, ColumnID, ColumnID,
					DisplayText,
					EListViewColumnType::BuiltIn,
					TEXT("Built-In"),
					bCanBeHidden,
					InDefaultVisibility == ESceneOutlinerColumnVisibility::Visible ? true : false
				}
			);
		};

		AddBuiltInColumn(
			ESceneOutlinerColumnVisibility::Visible, 2, FObjectMixerOutlinerVisibilityColumn::GetID(),
			FSceneOutlinerBuiltInColumnTypes::Gutter_Localized(), true,
			FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner)
			   {
				   return MakeShared<FObjectMixerOutlinerVisibilityColumn>(InSceneOutliner);
			   }
			)
		);

		AddBuiltInColumn(
			ESceneOutlinerColumnVisibility::Visible, 3, FObjectMixerOutlinerSoloColumn::GetID(),
			FObjectMixerOutlinerSoloColumn::GetLocalizedColumnName(), true,
			FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner)
				{
					return MakeShared<FObjectMixerOutlinerSoloColumn>(InSceneOutliner);
				}
			)
		);

		AddBuiltInColumn(
			ESceneOutlinerColumnVisibility::Visible, 10, FSceneOutlinerBuiltInColumnTypes::Label(),
			FSceneOutlinerBuiltInColumnTypes::Label_Localized(), false
		);

		AddBuiltInColumn(
			ESceneOutlinerColumnVisibility::Visible, 11, FSceneOutlinerBuiltInColumnTypes::ActorInfo(),
			FSceneOutlinerBuiltInColumnTypes::ActorInfo_Localized(), true			
		);
		
		// Sequencer Columns
		if (FObjectMixerEditorModule::Get().GetSequencers().Num() > 0)
		{
			const FText SpawnableDisplayText = LOCTEXT("SpawnableColumnName", "Spawnable");
			const FText SequencerInfoDisplayText = LOCTEXT("SequencerColumnName", "Sequencer");

			AddBuiltInColumn(
				ESceneOutlinerColumnVisibility::Visible, 12, SequencerSpawnableColumnName,
				SpawnableDisplayText, true,
				FCreateSceneOutlinerColumn::CreateRaw( &FLevelEditorSequencerIntegration::Get(), &FLevelEditorSequencerIntegration::CreateSequencerSpawnableColumn)
			);

			AddBuiltInColumn(
				ESceneOutlinerColumnVisibility::Visible, 13, SequencerInfoColumnName,
				SequencerInfoDisplayText, true,
				FCreateSceneOutlinerColumn::CreateRaw( &FLevelEditorSequencerIntegration::Get(), &FLevelEditorSequencerIntegration::CreateSequencerInfoColumn)
			);
		}

		ESceneOutlinerColumnVisibility UnsavedColumnVisibility = ESceneOutlinerColumnVisibility::Invisible;

		if (UWorld* WorldPtr = GetWorld())
		{
			CreateActorTextInfoColumns(OutInitOptions);
			
			if (WorldPtr->IsPartitionedWorld())
			{
				// We don't want the pinned column in non wp levels
				AddBuiltInColumn(
					ESceneOutlinerColumnVisibility::Visible, 0, FSceneOutlinerBuiltInColumnTypes::Pinned(),
					FSceneOutlinerBuiltInColumnTypes::Pinned_Localized(), true			
				);

				// We want the unsaved column to be visible by default in partitioned levels
				UnsavedColumnVisibility = ESceneOutlinerColumnVisibility::Visible;
			}
		}

		AddBuiltInColumn(
			UnsavedColumnVisibility, 1, FSceneOutlinerBuiltInColumnTypes::Unsaved(),
			FSceneOutlinerBuiltInColumnTypes::Unsaved_Localized(), true			
		);
	}

	// Add Property Columns to Map
	{
		TSet<UClass*> SpecifiedClasses;
		for (const TObjectPtr<UObjectMixerObjectFilter>& Instance : PinnedListModel->GetObjectFilterInstances())
		{
			SpecifiedClasses.Append(
				Instance->GetParentAndChildClassesFromSpecifiedClasses(
					PinnedListModel->ObjectClassesToFilterCache, PinnedListModel->PropertyInheritanceInclusionOptionsCache
				)
			);
		}
	
		for (const UClass* Class : SpecifiedClasses)
		{
			for (TFieldIterator<FProperty> FieldIterator(Class, EFieldIterationFlags::None); FieldIterator; ++FieldIterator)
			{
				if (FProperty* Property = *FieldIterator)
				{
					AddUniquePropertyColumnInfo(Property, PinnedListModel->bShouldIncludeUnsupportedPropertiesCache, PinnedListModel->ColumnsToExcludeCache);
				}
			}

			// Check Force Added Columns
			for (const FName& PropertyName : PinnedListModel->ForceAddedColumnsCache)
			{
				if (FProperty* Property = FindFProperty<FProperty>(Class, PropertyName))
				{
					AddUniquePropertyColumnInfo(Property, true);
				}
			}
		}
	}

	SortColumnsForHeaderContextMenu();

	// Add Property Columns to Header
	{
		for (const FObjectMixerSceneOutlinerColumnInfo& ColumnInfo : HeaderColumnInfos)
		{
			// Builtin columns are already in Header
			if (ColumnInfo.PropertyType == EListViewColumnType::BuiltIn)
			{
				continue;
			}
			
			const FText PropertyName = ColumnInfo.PropertyDisplayText;

			if (PropertyName.IsEmpty())
			{
				continue;
			}
			
			OutInitOptions.ColumnMap.Add(
				ColumnInfo.ColumnID,
				FSceneOutlinerColumnInfo(
					ColumnInfo.bIsDesiredToBeShownByDefault ? ESceneOutlinerColumnVisibility::Visible : ESceneOutlinerColumnVisibility::Invisible, 30,
					FCreateSceneOutlinerColumn::CreateLambda([Property = ColumnInfo.PropertyRef](ISceneOutliner& InSceneOutliner)
					{
						return MakeShared<FObjectMixerOutlinerPropertyColumn>(InSceneOutliner, Property);
					}),
					true,
					TOptional<float>(),
					PropertyName,
					EHeaderComboVisibility::Never,
					FOnGetContent::CreateRaw(this, &SObjectMixerEditorList::GenerateHeaderRowContextMenu)
				)
			);
		}
	}
}

void SObjectMixerEditorList::SortColumnsForHeaderContextMenu()
{
	// Alphabetical sort by Property Name
	HeaderColumnInfos.StableSort([](const FObjectMixerSceneOutlinerColumnInfo& A, const FObjectMixerSceneOutlinerColumnInfo& B)
	{
		return A.PropertyDisplayText.ToString() < B.PropertyDisplayText.ToString();
	});

	// Alphabetical sort by Property Category Name
	HeaderColumnInfos.StableSort([](const FObjectMixerSceneOutlinerColumnInfo& A, const FObjectMixerSceneOutlinerColumnInfo& B)
	{
		return A.PropertyCategoryName.LexicalLess(B.PropertyCategoryName);
	});

	// Sort by type
	HeaderColumnInfos.StableSort([](const FObjectMixerSceneOutlinerColumnInfo& A, const FObjectMixerSceneOutlinerColumnInfo& B)
	{
		return A.PropertyType < B.PropertyType;
	});
}

void SObjectMixerEditorList::PropagatePropertyChangesToSelectedRows()
{
	using LambdaType = void(*)(const TSet<TWeakPtr<ISceneOutlinerTreeItem>>&,
			TSharedPtr<STreeView<TSharedPtr<ISceneOutlinerTreeItem>>>,
			TSet<FObjectMixerEditorListRowData::FPropertyPropagationInfo>&);
	
	static LambdaType RecursivelyPropagatePropertyChangesToSelectedRows = [](
		const TSet<TWeakPtr<ISceneOutlinerTreeItem>>& InObjects,
		TSharedPtr<STreeView<TSharedPtr<ISceneOutlinerTreeItem>>> InTreeViewPtr,
		TSet<FObjectMixerEditorListRowData::FPropertyPropagationInfo>& InPendingPropagations)
	{
		for (const TWeakPtr<ISceneOutlinerTreeItem>& TreeViewItem : InObjects)
		{
			if (TSharedPtr<ISceneOutlinerTreeItem> PinnedItem = TreeViewItem.Pin())
			{
				if (FObjectMixerEditorListRowData* RowData = FObjectMixerUtils::GetRowData(PinnedItem))
				{
					const FSceneOutlinerTreeItemID TreeItemUniqueIdentifier = PinnedItem->GetID();

					if (const FObjectMixerEditorListRowData::FPropertyPropagationInfo* Match =
						Algo::FindByPredicate(
							InPendingPropagations,
							[&TreeItemUniqueIdentifier](const FObjectMixerEditorListRowData::FPropertyPropagationInfo& Other)
							{
								return Other.RowIdentifier == TreeItemUniqueIdentifier;
							}))
					{
						
						if (RowData->PropagateChangesToSimilarSelectedRowProperties(PinnedItem.ToSharedRef(), *Match))
						{
							// Only remove this propagation from the list if it was successfully propagated (or there is no work to do)
							InPendingPropagations.Remove(*Match);
						}
					}
				}

				RecursivelyPropagatePropertyChangesToSelectedRows(
					PinnedItem->GetChildren(), InTreeViewPtr, InPendingPropagations);
			}
		}
	};

	if (GetSelectedTreeViewItemCount() > 1)
	{
		RecursivelyPropagatePropertyChangesToSelectedRows(
			GetWeakTreeRootItems(), GetTreeView(), PendingPropertyPropagations);
	}
}

#undef LOCTEXT_NAMESPACE
