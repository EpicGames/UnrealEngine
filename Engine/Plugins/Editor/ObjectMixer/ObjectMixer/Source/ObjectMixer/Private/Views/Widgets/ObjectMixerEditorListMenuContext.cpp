// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Widgets/ObjectMixerEditorListMenuContext.h"

#include "Views/List/ObjectMixerEditorList.h"
#include "Views/List/ObjectMixerUtils.h"

#include "Algo/Find.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectMixerEditorListMenuContext)

#define LOCTEXT_NAMESPACE "ObjectMixerEditor"

bool UObjectMixerEditorListMenuContext::DoesSelectionHaveType(const FObjectMixerEditorListMenuContextData& InData, UClass* Type)
{
	if (const TSharedPtr<ISceneOutlinerTreeItem>* Match = Algo::FindByPredicate(
		InData.SelectedItems,
		[Type](const TSharedPtr<ISceneOutlinerTreeItem>& SelectedItem)
		{
			const UObject* Object = FObjectMixerUtils::GetRowObject(SelectedItem);
			return Object && Object->IsA(Type);
		}))
	{
		return true;
	}

	return false;
}

void UObjectMixerEditorListMenuContext::RegisterFoldersOnlyContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered("ObjectMixer.FoldersOnlyContextMenu"))
	{
		return;
	}

	UToolMenu* Menu = ToolMenus->RegisterMenu("ObjectMixer.FoldersOnlyContextMenu");
	Menu->AddDynamicSection("FolderContextMenuDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		// Ensure proper context
		UObjectMixerEditorListMenuContext* Context = InMenu->FindContext<UObjectMixerEditorListMenuContext>();
		if (!Context || Context->Data.SelectedItems.Num() == 0)
		{
			return;
		}

		const FObjectMixerEditorListMenuContextData& ContextData = Context->Data;

		// For future CL
		// {
		// 	FToolMenuSection& Section = InMenu->AddSection("ElementEditActions");
		// 	Section.AddSubMenu(
		// 		"EditSubMenu",
		// 		LOCTEXT("EditSubMenu", "Edit"),
		// 		FText::GetEmpty(),
		// 		FNewToolMenuDelegate::CreateLambda([ContextData](UToolMenu* InMenu)
		// 		{
		// 			FToolMenuSection& Section = InMenu->AddSection("Section");
		// 			Section.AddEntry(MakeCustomEditMenu(ContextData));
		// 		}),
		// 		false, // default value
		// 		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"));
		// }
	}));
}

void UObjectMixerEditorListMenuContext::AddCollectionsMenuItem(UToolMenu* InMenu, const FObjectMixerEditorListMenuContextData& ContextData)
{
	FToolMenuSection& Section = InMenu->FindOrAddSection("ObjectMixerCollections");
	Section.Label = LOCTEXT("ObjectMixerCollectionsSectionName", "Mixer Collections");
		
	Section.AddSubMenu(
		"SelectCollectionsSubMenu",
		LOCTEXT("SelectCollectionsSubmenu", "Select or Add Collection"),
		LOCTEXT("SelectCollectionsSubmenu_Tooltip", "Select the collection to which you wish to assign this object."),
		FNewToolMenuDelegate::CreateStatic(&UObjectMixerEditorListMenuContext::CreateSelectCollectionsSubMenu, ContextData)
	);
}

void UObjectMixerEditorListMenuContext::RegisterObjectMixerDynamicCollectionsContextMenuExtension(const FName& MenuName)
{
	if (UToolMenu* ActorContextMenu = UToolMenus::Get()->ExtendMenu(MenuName))
	{		
		ActorContextMenu->AddDynamicSection("DynamicCollectionsSection", FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InMenu)
			{
				// Ensure proper context
				UObjectMixerEditorListMenuContext* Context = InMenu->FindContext<UObjectMixerEditorListMenuContext>();
				if (!Context || Context->Data.SelectedItems.Num() == 0)
				{
					return;
				}

				const FObjectMixerEditorListMenuContextData& ContextData = Context->Data;

				AddCollectionsMenuItem(InMenu, ContextData);
			}),
			FToolMenuInsert(NAME_None,EToolMenuInsertType::First)
		);
	}
}

void UObjectMixerEditorListMenuContext::AddCollectionWidget(const FName& Key, const FObjectMixerEditorListMenuContextData& ContextData, UToolMenu* Menu)
{
	const FText KeyText = FText::FromName(Key);

	TSharedRef<SHorizontalBox> Widget = SNew(SHorizontalBox);

	Widget->AddSlot()
	.Padding(FMargin(8, 0))
	.AutoWidth()
	[
		SNew(SCheckBox)
		.OnCheckStateChanged_Lambda([Key, ContextData](ECheckBoxState)
		{
			OnCollectionMenuEntryCheckStateChanged(Key, ContextData);
		})
		.IsChecked_Static(&UObjectMixerEditorListMenuContext::GetCheckStateForCollection, Key, ContextData)
	];

	Widget->AddSlot()
	.AutoWidth()
	[
		SNew(STextBlock)
		.Text(KeyText)
	];
								
	Menu->AddMenuEntry(Key, FToolMenuEntry::InitWidget(Key, Widget, FText(), true));
}

void UObjectMixerEditorListMenuContext::CreateSelectCollectionsSubMenu(UToolMenu* Menu, FObjectMixerEditorListMenuContextData ContextData)
{
	FToolMenuEntry Args;
	Args.Type = EMultiBlockType::Widget;
	Args.MakeCustomWidget.BindLambda(
		[ContextData](const FToolMenuContext&, const FToolMenuCustomWidgetContext&)
		{
			return
				SNew(SBox)
				.MinDesiredWidth(200)
				.Padding(8, 0)
				[
					SNew(SEditableTextBox)
					.HintText(LOCTEXT("NewCollectionEditableHintText", "Enter a new collection name..."))
					.OnTextCommitted_Static(&UObjectMixerEditorListMenuContext::OnTextCommitted, ContextData)
				]
			;
		}
	);
	Menu->AddMenuEntry("NewCollectionInput", Args);
						
	if (const TSharedPtr<FObjectMixerEditorList> List = ContextData.ListModelPtr.Pin())
	{
		if (TArray<FName> Collections = List->GetAllCollectionNames(); Collections.Num() > 0)
		{
			FToolMenuSection& Section = Menu->FindOrAddSection("Collections");
			FToolMenuEntry& Separator = Section.AddSeparator("CollectionsSeparator");
			Separator.Label = LOCTEXT("CollectionsSeparatorLabel", "Collections");

			Collections.StableSort([](const FName& A, const FName B)
			{
				return A.LexicalLess(B);
			});
							
			for (const FName& Key : Collections)
			{
				AddCollectionWidget(Key, ContextData, Menu);
			}
		}
	}
}

void UObjectMixerEditorListMenuContext::OnTextCommitted(const FText& InText, ETextCommit::Type InCommitType, const FObjectMixerEditorListMenuContextData ContextData)
{
	if (InCommitType == ETextCommit::OnEnter)
	{
		AddObjectsToCollection(*InText.ToString(), ContextData);
	}
}

void UObjectMixerEditorListMenuContext::OnCollectionMenuEntryCheckStateChanged(const FName Key, const FObjectMixerEditorListMenuContextData ContextData)
{
	if (AreAllObjectsInCollection(Key, ContextData))
	{
		RemoveObjectsFromCollection(Key, ContextData);
	}
	else
	{
		AddObjectsToCollection(Key, ContextData);
	}
}

void UObjectMixerEditorListMenuContext::AddObjectsToCollection(const FName Key, const FObjectMixerEditorListMenuContextData ContextData)
{
	if (const TSharedPtr<FObjectMixerEditorList> List = ContextData.ListModelPtr.Pin())
	{
		TSet<FSoftObjectPath> ObjectPaths;

		for (const TSharedPtr<ISceneOutlinerTreeItem>& Item : ContextData.SelectedItems)
		{
			if (const UObject* Object = FObjectMixerUtils::GetRowObject(Item))
			{
				ObjectPaths.Add(Object);
			}
		}
		
		List->RequestAddObjectsToCollection(Key, ObjectPaths);
	}
}

void UObjectMixerEditorListMenuContext::RemoveObjectsFromCollection(const FName Key, const FObjectMixerEditorListMenuContextData ContextData)
{
	if (const TSharedPtr<FObjectMixerEditorList> List = ContextData.ListModelPtr.Pin())
	{
		TSet<FSoftObjectPath> ObjectPaths;

		for (const TSharedPtr<ISceneOutlinerTreeItem>& Item : ContextData.SelectedItems)
		{
			if (const UObject* Object = FObjectMixerUtils::GetRowObject(Item))
			{
				ObjectPaths.Add(Object);
			}
		}
		
		List->RequestRemoveObjectsFromCollection(Key, ObjectPaths);
	}
}

bool UObjectMixerEditorListMenuContext::AreAllObjectsInCollection(const FName Key, const FObjectMixerEditorListMenuContextData ContextData)
{
	bool bAreAllSelectedObjectsInCollection = false;

	if (const TSharedPtr<FObjectMixerEditorList> List = ContextData.ListModelPtr.Pin())
	{
		for (const TSharedPtr<ISceneOutlinerTreeItem>& Item : ContextData.SelectedItems)
		{
			if (const UObject* Object = FObjectMixerUtils::GetRowObject(Item))
			{
				bAreAllSelectedObjectsInCollection = List->IsObjectInCollection(Key, Object);

				if (!bAreAllSelectedObjectsInCollection)
				{
					return false;
				}
			}
		}
	}

	return bAreAllSelectedObjectsInCollection;
}

ECheckBoxState UObjectMixerEditorListMenuContext::GetCheckStateForCollection(const FName Key,
	const FObjectMixerEditorListMenuContextData ContextData)
{
	const int32 ItemCount = ContextData.SelectedItems.Num();
	int32 ItemsInCollection = 0;
	int32 ItemsNotInCollection = 0;

	if (const TSharedPtr<FObjectMixerEditorList> List = ContextData.ListModelPtr.Pin())
	{
		for (const TSharedPtr<ISceneOutlinerTreeItem>& Item : ContextData.SelectedItems)
		{
			if (const UObject* Object = FObjectMixerUtils::GetRowObject(Item))
			{
				if (List->IsObjectInCollection(Key, Object))
				{
					ItemsInCollection++;
				}
				else
				{
					ItemsNotInCollection++;
				}
			}
		}
	}

	ECheckBoxState ReturnValue = ECheckBoxState::Undetermined;

	if (ItemsInCollection == ItemCount)
	{
		ReturnValue = ECheckBoxState::Checked;
	}
	else if (ItemsNotInCollection == ItemCount)
	{
		ReturnValue = ECheckBoxState::Unchecked;
	}

	return ReturnValue;
}

FToolMenuEntry UObjectMixerEditorListMenuContext::MakeCustomEditMenu(const FObjectMixerEditorListMenuContextData& ContextData)
{
	FToolMenuEntry Entry;
	Entry.Name = TEXT("ObjectMixerGenericCommands");
	Entry.Type = EMultiBlockType::Widget;
	FToolMenuEntryWidgetData WidgetData;
	WidgetData.bNoIndent = true;
	WidgetData.bNoPadding = true;
	Entry.WidgetData = WidgetData;
	Entry.MakeCustomWidget.BindLambda(
		[ContextData](const FToolMenuContext&, const FToolMenuCustomWidgetContext&)
		{
			if (TSharedPtr<FObjectMixerEditorList> PinnedList = ContextData.ListModelPtr.Pin())
			{
				// Add options with our mapped command list to an FMenuBuilder
				FMenuBuilder Builder(true, PinnedList->ObjectMixerElementEditCommands);
				{
					Builder.AddMenuEntry( FGenericCommands::Get().Cut);
					Builder.AddMenuEntry( FGenericCommands::Get().Copy);
					Builder.AddMenuEntry( FGenericCommands::Get().Paste);
					Builder.AddMenuEntry( FGenericCommands::Get().Duplicate);
					Builder.AddMenuEntry( FGenericCommands::Get().Delete);
					Builder.AddMenuEntry( FGenericCommands::Get().Rename);
				}

				return Builder.MakeWidget();
			}

			return SNullWidget::NullWidget;
		}
	);

	return Entry;
}

void UObjectMixerEditorListMenuContext::ReplaceEditSubMenu(const FObjectMixerEditorListMenuContextData& ContextData)
{
	if (UToolMenu* EditSubMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorSceneOutliner.ContextMenu.ElementEditActions"))
	{
		// Remove existing edit sub menu options
		FCustomizedToolMenu* MenuCustomization = EditSubMenu->AddMenuCustomization();
		{
			MenuCustomization->AddEntry(TEXT("Cut"))->Visibility = ECustomizedToolMenuVisibility::Hidden;
			MenuCustomization->AddEntry(TEXT("Copy"))->Visibility = ECustomizedToolMenuVisibility::Hidden;
			MenuCustomization->AddEntry(TEXT("Paste"))->Visibility = ECustomizedToolMenuVisibility::Hidden;
			MenuCustomization->AddEntry(TEXT("Duplicate"))->Visibility = ECustomizedToolMenuVisibility::Hidden;
			MenuCustomization->AddEntry(TEXT("Delete"))->Visibility = ECustomizedToolMenuVisibility::Hidden;
			MenuCustomization->AddEntry(TEXT("Rename"))->Visibility = ECustomizedToolMenuVisibility::Hidden;
		}

		// Add our menu widget to the edit submenu
		// It must be a widget entry because we hid the entries above, so they won't show as regular menu entries since they share names
		EditSubMenu->AddMenuEntry(NAME_None, MakeCustomEditMenu(ContextData));
	}
}

#undef LOCTEXT_NAMESPACE
