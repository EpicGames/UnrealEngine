// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Widgets/ObjectMixerEditorUWidget.h"

#include "ObjectMixerEditorModule.h"

#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectMixerEditorUWidget)

#define LOCTEXT_NAMESPACE "ObjectMixerEditor"

const FText UObjectMixerEditorUWidget::GetPaletteCategory()
{
	return LOCTEXT("Editor", "Editor");
}

TSharedRef<SWidget> UObjectMixerEditorUWidget::RebuildWidget()
{
	TSharedRef<SBorder> Border = SNew(SBorder)
		.Padding(0.0f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
	;
	
	if (GEditor)
	{
		if (!GEditor->IsPlayingSessionInEditor())
		{
			const TSharedPtr<SWidget> Widget =
				FObjectMixerEditorModule::Get().MakeObjectMixerDialog(ObjectMixerWidgetUserConfig.DefaultFilterClass);
			Border->SetContent(Widget.IsValid() ? Widget.ToSharedRef() : SNullWidget::NullWidget);

			return Border;
		}
	}

	Border->SetContent(
		SNew(STextBlock)
		.Text(LOCTEXT("EditorWidget", "Editor Only"))
	);

	return Border;
}

#undef LOCTEXT_NAMESPACE
