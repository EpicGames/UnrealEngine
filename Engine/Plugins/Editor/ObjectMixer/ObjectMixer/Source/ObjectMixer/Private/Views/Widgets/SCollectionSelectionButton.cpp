// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Widgets/SCollectionSelectionButton.h"

#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "Views/List/ObjectMixerEditorList.h"
#include "Views/List/SObjectMixerEditorList.h"

#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "ObjectMixerEditor"

class FCollectionSelectionButtonDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FCollectionSelectionButtonDragDropOp, FDecoratedDragDropOp)

	/** The item being dragged and dropped */
	FName DraggedItem;

	/** Constructs a new drag/drop operation */
	static TSharedRef<FCollectionSelectionButtonDragDropOp> New(const FName DraggedItem)
	{
		TSharedRef<FCollectionSelectionButtonDragDropOp> Operation = MakeShareable(
			   new FCollectionSelectionButtonDragDropOp());
		if (DraggedItem != NAME_None)
		{
			Operation->DraggedItem = DraggedItem;

			Operation->DefaultHoverIcon = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.Error");

			Operation->DefaultHoverText = LOCTEXT("DefaultCollectionButtonHoverText","Drop onto another Collection Button to set a custom order.");

			Operation->Construct();
		}

		return Operation;
	}
};

void SCollectionSelectionButton::Construct(
	const FArguments& InArgs,
	const TSharedRef<SObjectMixerEditorList> ListWidget,
	const FName InCollectionName)
{
	ListWidgetPtr = ListWidget;
	CollectionName = InCollectionName;
	
	ChildSlot
	[
		SAssignNew(MenuAnchor, SMenuAnchor)
		.OnGetMenuContent(this, &SCollectionSelectionButton::GetContextMenu)
		[
			SNew(SBorder)
			.Padding(FMargin(16, 4))
			.BorderImage(this, &SCollectionSelectionButton::GetBorderBrush)
			.ForegroundColor(this, &SCollectionSelectionButton::GetBorderForeground)
			[
				SAssignNew(EditableTextBlock, SInlineEditableTextBlock)
				.Style(&FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockSmallStyle"))
				.Text(FText::FromName(CollectionName))
				.Visibility(EVisibility::HitTestInvisible)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.OnTextCommitted(this, &SCollectionSelectionButton::OnEditableTextCommitted)
			]
		]
	];
}

TSharedRef<SWidget> SCollectionSelectionButton::GetContextMenu() const
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("DeleteCollectionButtonMenuEntry", "Delete '{0}'"), FText::FromName(CollectionName)),
		LOCTEXT("DeleteCollectionButtonMenuEntryTooltip", "Remove this collection. This operation can be undone."),
		FGenericCommands::Get().Delete->GetIcon(),
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			ListWidgetPtr.Pin()->RequestRemoveCollection(CollectionName);
		}))
	);

	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("DuplicateCollectionButtonMenuEntry", "Duplicate '{0}'"), FText::FromName(CollectionName)),
		LOCTEXT("DuplicateCollectionButtonMenuEntryTooltip", "Duplicate this collection. This operation can be undone."),
		FGenericCommands::Get().Duplicate->GetIcon(),
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			FName DuplicateName = CollectionName;
			ListWidgetPtr.Pin()->RequestDuplicateCollection(CollectionName, DuplicateName);
		}))
	);

	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("RenameCollectionButtonMenuEntry", "Rename '{0}'"), FText::FromName(CollectionName)),
		LOCTEXT("RenameCollectionButtonMenuEntryTooltip", "Rename this collection. This operation can be undone."),
		FGenericCommands::Get().Rename->GetIcon(),
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			EditableTextBlock->EnterEditingMode();
		}))
	);
			
	return MenuBuilder.MakeWidget();
}

void SCollectionSelectionButton::OnEditableTextCommitted(const FText& Text, ETextCommit::Type Type)
{
	if (Type == ETextCommit::OnEnter)
	{
		const FName NewCollectionName = *Text.ToString();
		ListWidgetPtr.Pin()->RequestRenameCollection(CollectionName, NewCollectionName);
	}
}

FReply SCollectionSelectionButton::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) 
{
	bIsPressed = true;
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		return FReply::Handled().DetectDrag( SharedThis(this), EKeys::LeftMouseButton );
	}
	return FReply::Handled();
}

FReply SCollectionSelectionButton::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) 
{
	bIsPressed = false;
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		ListWidgetPtr.Pin()->OnCollectionCheckedStateChanged(!GetIsSelected(), CollectionName);
	}
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (CollectionName != UObjectMixerEditorSerializedData::AllCollectionName && CollectionName != NAME_None)
		{
			MenuAnchor->SetIsOpen(true);
		}
	}
	return FReply::Handled();
}

FReply SCollectionSelectionButton::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (CollectionName != UObjectMixerEditorSerializedData::AllCollectionName && CollectionName != NAME_None)
	{
		TSharedRef<FCollectionSelectionButtonDragDropOp> OperationFromCollection =
		   FCollectionSelectionButtonDragDropOp::New(CollectionName);

		OperationFromCollection->ResetToDefaultToolTip();

		bDropIsValid = false;

		return FReply::Handled().BeginDragDrop(OperationFromCollection);
	}

	return FReply::Handled();
}

void SCollectionSelectionButton::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FSceneOutlinerDragDropOp> SceneOutlinerOperation = DragDropEvent.GetOperationAs<FSceneOutlinerDragDropOp>())
	{
		if (CollectionName != UObjectMixerEditorSerializedData::AllCollectionName && CollectionName != NAME_None)
		{
			SceneOutlinerOperation->SetToolTip(
			   LOCTEXT("DropRowItemsOntoCollectionButtonCTA","Add selected items to this collection"),
			   FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.OK"));

			bDropIsValid = true;
		}
	}
	else if (TSharedPtr<FActorDragDropGraphEdOp> ActorOnlyOperation = DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>())
	{
		if (CollectionName != UObjectMixerEditorSerializedData::AllCollectionName && CollectionName != NAME_None)
		{
			ActorOnlyOperation->SetToolTip(
				FActorDragDropGraphEdOp::ToolTip_Compatible, 
			    LOCTEXT("DropRowActorsOntoCollectionButtonCTA","Add selected actors to this collection"));

			bDropIsValid = true;
		}
	}
	else if (TSharedPtr<FCollectionSelectionButtonDragDropOp> OperationFromCollection =
		DragDropEvent.GetOperationAs<FCollectionSelectionButtonDragDropOp>())
	{
		if (CollectionName != UObjectMixerEditorSerializedData::AllCollectionName &&
			CollectionName != NAME_None && CollectionName != OperationFromCollection->DraggedItem)
		{
			OperationFromCollection->SetToolTip(
			   FText::Format(
				LOCTEXT("DropCollectionButtonOntoCollectionButtonCTA_Format","Reorder {0} before {1}"),
				FText::FromName(OperationFromCollection->DraggedItem), FText::FromName(CollectionName)),
			   FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.OK"));

			bDropIsValid = true;
		}
	}
}

void SCollectionSelectionButton::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	bIsPressed = false;
	bDropIsValid = false;
	
	if (TSharedPtr<FSceneOutlinerDragDropOp> OperationFromRow =
		DragDropEvent.GetOperationAs<FSceneOutlinerDragDropOp>())
	{
		OperationFromRow->ResetToDefaultToolTip();
	}
	else if (TSharedPtr<FActorDragDropGraphEdOp> ActorOperation =
		DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>())
	{
		ActorOperation->ResetToDefaultToolTip();
	}
	else if (TSharedPtr<FCollectionSelectionButtonDragDropOp> OperationFromCollection =
		DragDropEvent.GetOperationAs<FCollectionSelectionButtonDragDropOp>())
	{
		OperationFromCollection->ResetToDefaultToolTip();
	}
}

FReply SCollectionSelectionButton::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bIsPressed = false;
	bDropIsValid = false;
	
	if (TSharedPtr<FSceneOutlinerDragDropOp> OperationFromRow =
		DragDropEvent.GetOperationAs<FSceneOutlinerDragDropOp>())
	{
		if (CollectionName != UObjectMixerEditorSerializedData::AllCollectionName && CollectionName != NAME_None)
		{
			if (const TSharedPtr<FObjectMixerEditorList> ListModel = ListWidgetPtr.Pin()->GetListModelPtr().Pin())
			{
				TSet<FSoftObjectPath> ObjectPaths;

				for (TWeakObjectPtr<AActor> Item : OperationFromRow->GetSubOp<FActorDragDropOp>()->Actors)
				{
					if (Item.IsValid())
					{
						ObjectPaths.Add(Item.Get());
					}
				}
	
				ListModel->RequestAddObjectsToCollection(CollectionName, ObjectPaths);
			}
		}
	}
	else if (TSharedPtr<FActorDragDropGraphEdOp> ActorOperation =
		DragDropEvent.GetOperationAs<FActorDragDropGraphEdOp>())
	{
		if (CollectionName != UObjectMixerEditorSerializedData::AllCollectionName && CollectionName != NAME_None)
		{
			if (const TSharedPtr<FObjectMixerEditorList> ListModel = ListWidgetPtr.Pin()->GetListModelPtr().Pin())
			{
				TSet<FSoftObjectPath> ObjectPaths;

				for (TWeakObjectPtr<AActor> Item : ActorOperation->Actors)
				{
					if (Item.IsValid())
					{
						ObjectPaths.Add(Item.Get());
					}
				}
	
				ListModel->RequestAddObjectsToCollection(CollectionName, ObjectPaths);
			}
		}
	}
	else if (TSharedPtr<FCollectionSelectionButtonDragDropOp> OperationFromCollection =
		DragDropEvent.GetOperationAs<FCollectionSelectionButtonDragDropOp>())
	{
		if (CollectionName != UObjectMixerEditorSerializedData::AllCollectionName &&
			CollectionName != NAME_None && CollectionName != OperationFromCollection->DraggedItem)
		{
			if (const TSharedPtr<FObjectMixerEditorList> ListModel = ListWidgetPtr.Pin()->GetListModelPtr().Pin())
			{
				ListModel->RequestReorderCollection(
					OperationFromCollection->DraggedItem, CollectionName);
			}
		}
	}
		
	return FReply::Handled();
}

const FSlateBrush* SCollectionSelectionButton::GetBorderBrush() const
{
	if (GetIsSelected())
	{
		if (bIsPressed)
		{
			return &CheckedPressedImage;
		}
		else if (IsHovered())
		{
			return &CheckedHoveredImage;
		}

		return &CheckedImage;
	}

	if (bIsPressed)
	{
		return &UncheckedPressedImage;
	}
	else if (IsHovered())
	{
		return &UncheckedHoveredImage;
	}

	return &UncheckedImage;
			
}

FSlateColor SCollectionSelectionButton::GetBorderForeground() const
{
	if (GetIsSelected() || bIsPressed || IsHovered())
	{
		return FStyleColors::White;
	}

	return FStyleColors::Foreground;
}

bool SCollectionSelectionButton::GetIsSelected() const
{
	return ListWidgetPtr.Pin()->IsCollectionSelected(CollectionName);
}

#undef LOCTEXT_NAMESPACE
