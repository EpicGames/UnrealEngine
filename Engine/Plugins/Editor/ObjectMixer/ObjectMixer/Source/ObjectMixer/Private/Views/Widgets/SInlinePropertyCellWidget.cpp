// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/Widgets/SInlinePropertyCellWidget.h"

#include "ObjectMixerEditorModule.h"
#include "Views/List/ObjectMixerEditorListRowData.h"
#include "Views/List/ObjectMixerUtils.h"

#include "ISequencer.h"
#include "ISinglePropertyView.h"
#include "KeyParams.h"
#include "KeyPropertyParams.h"
#include "SSimpleButton.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Layout/WidgetPath.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "ObjectMixerEditor"

namespace UE::ObjectMixer
{
	void CreateKey(UObject* InObject, TSharedPtr<IPropertyHandle> InPropertyHandle)
	{
		if (!IsValid(InObject) || !InPropertyHandle.IsValid() || !InPropertyHandle->IsValidHandle())
		{
			return;
		}

		for (TWeakPtr<ISequencer> Sequencer : FObjectMixerEditorModule::Get().GetSequencers())
		{
			if (!Sequencer.IsValid())
			{
				continue;
			}

			const FKeyPropertyParams KeyPropertyParams({InObject}, *InPropertyHandle, ESequencerKeyMode::ManualKeyForced);
			Sequencer.Pin()->KeyProperty(KeyPropertyParams);
		}
	}
}
	
void SInlinePropertyCellWidget::Construct(
	const FArguments& InArgs, const FName InColumnName, const TSharedRef<ISceneOutlinerTreeItem> RowPtr)
{
	const bool bGetHybridComponent = true;
	UObject* ObjectRef = FObjectMixerUtils::GetRowObject(RowPtr, bGetHybridComponent);
	if (!ObjectRef || InColumnName.IsEqual(NAME_None))
	{
		return;
	}

	ColumnName = InColumnName;
	WeakRowPtr = RowPtr;
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FSinglePropertyParams Params;
	Params.NamePlacement = EPropertyNamePlacement::Hidden;
	
	const TSharedPtr<ISinglePropertyView> SinglePropertyView =
		PropertyEditorModule.CreateSingleProperty(ObjectRef, InColumnName, Params
	);

	if (SinglePropertyView)
	{
		const TSharedPtr<IPropertyHandle> Handle = SinglePropertyView->GetPropertyHandle();
		
		if (Handle && Handle->IsValidHandle())
		{
			FObjectMixerUtils::GetRowData(RowPtr)->PropertyNamesToHandles.Add(InColumnName, Handle);
				
			// Simultaneously edit all selected rows with a similar property
			if (InArgs._OnPropertyValueChanged.IsBound())
			{
				Handle->SetOnPropertyValueChangedWithData(InArgs._OnPropertyValueChanged);
				Handle->SetOnChildPropertyValueChangedWithData(InArgs._OnPropertyValueChanged);
			}

			ChildSlot
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SinglePropertyView.ToSharedRef()
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SSimpleButton)
					.ToolTipText(LOCTEXT("CreateKeyToolTip", "Add a keyframe for this property."))
					.OnClicked_Lambda([ObjectRef, Handle]()
					{
						UE::ObjectMixer::CreateKey(ObjectRef, Handle);
						return FReply::Handled();
					})
					.Visibility(FObjectMixerEditorModule::Get().GetSequencers().Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed)
					.Icon(FAppStyle::Get().GetBrush("Sequencer.AddKey.Details"))
				]
			];
		}
	}
}

FReply SInlinePropertyCellWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() ==  EKeys::RightMouseButton)
	{
		return MakePropertyContextMenu(MouseEvent);
	}
	
	return SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
}

FReply SInlinePropertyCellWidget::MakePropertyContextMenu(const FPointerEvent& MouseEvent)
{
	FMenuBuilder MenuBuilder(true, nullptr, nullptr, true);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyProperty", "Copy"),
		LOCTEXT("CopyProperty_ToolTip", "Copy this property value"),
		FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
		FExecuteAction::CreateRaw(this, &SInlinePropertyCellWidget::CopyPropertyValue));

	FUIAction PasteAction = 
		FUIAction(
			FExecuteAction::CreateRaw(this, &SInlinePropertyCellWidget::PastePropertyValue),
			FCanExecuteAction::CreateRaw(this, &SInlinePropertyCellWidget::CanPaste)
		)
	;

	MenuBuilder.AddMenuEntry(
		LOCTEXT("PasteProperty", "Paste"),
		LOCTEXT("PasteProperty_ToolTip", "Paste the copied value here"),
		FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Paste"),
		PasteAction);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyPropertyDisplayName", "Copy Display Name"),
		LOCTEXT("CopyPropertyDisplayName_ToolTip", "Copy this property display name"),
		FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
		FExecuteAction::CreateRaw(this, &SInlinePropertyCellWidget::CopyPropertyName));

	const FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

	FSlateApplication::Get().PushMenu(
		AsShared(), WidgetPath, MenuBuilder.MakeWidget(),
		MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect::ContextMenu
	);

	return FReply::Handled();
}

void SInlinePropertyCellWidget::CopyPropertyValue() const
{
	check(WeakRowPtr.IsValid());
			
	const TWeakPtr<IPropertyHandle>* PropertyHandle = FObjectMixerUtils::GetRowData(WeakRowPtr.Pin())->PropertyNamesToHandles.Find(ColumnName);
	if (PropertyHandle && PropertyHandle->IsValid())
	{
		FString Value;
		const TSharedPtr<IPropertyHandle> PinnedHandle = PropertyHandle->Pin();
		if (PinnedHandle->GetValueAsFormattedString(Value, PPF_Copy) == FPropertyAccess::Success)
		{
			FPlatformApplicationMisc::ClipboardCopy(*Value);
		}
	}
}

void SInlinePropertyCellWidget::PastePropertyValue() const
{
	check(WeakRowPtr.IsValid());
			
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
			
	const TWeakPtr<IPropertyHandle>* PropertyHandle = FObjectMixerUtils::GetRowData(WeakRowPtr.Pin())->PropertyNamesToHandles.Find(ColumnName);
	if (!ClipboardContent.IsEmpty() && PropertyHandle && PropertyHandle->IsValid())
	{
		(*PropertyHandle).Pin()->SetValueFromFormattedString(ClipboardContent, EPropertyValueSetFlags::InstanceObjects);
	}
}

bool SInlinePropertyCellWidget::CanPaste() const
{
	check(WeakRowPtr.IsValid());
			
	FString ClipboardContent;
			
	const TWeakPtr<IPropertyHandle>* PropertyHandle = FObjectMixerUtils::GetRowData(WeakRowPtr.Pin())->PropertyNamesToHandles.Find(ColumnName);
	if (PropertyHandle && PropertyHandle->IsValid())
	{
		if ((*PropertyHandle).Pin()->IsEditConst())
		{
			return false;
		}
	}

	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	return !ClipboardContent.IsEmpty();
}

void SInlinePropertyCellWidget::CopyPropertyName() const
{
	check(WeakRowPtr.IsValid());
			
	const TWeakPtr<IPropertyHandle>* PropertyHandle = FObjectMixerUtils::GetRowData(WeakRowPtr.Pin())->PropertyNamesToHandles.Find(ColumnName);
	if (PropertyHandle && PropertyHandle->IsValid())
	{
		const FString Value = (*PropertyHandle).Pin()->GetPropertyDisplayName().ToString();
		if (!Value.IsEmpty())
		{
			FPlatformApplicationMisc::ClipboardCopy(*Value);
		}
	}
}

#undef LOCTEXT_NAMESPACE
