// Copyright Epic Games, Inc. All Rights Reserved.

#include "SObjectMixerPlacementAssetMenuEntry.h"

#include "AssetSelection.h"
#include "ClassIconFinder.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "EditorClassUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "IPlacementModeModule.h"
#include "LevelEditorActions.h"
#include "LevelEditorViewport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SToolTip.h"

void SObjectMixerPlacementAssetMenuEntry::Construct(const FArguments& InArgs, const TSharedPtr<const FPlaceableItem>& InItem)
{	
	bIsPressed = false;

	check(InItem.IsValid());

	Item = InItem;

	AssetImage = nullptr;

	TSharedPtr< SHorizontalBox > ActorType = SNew( SHorizontalBox );

	const bool bIsClass = Item->AssetData.GetClass() == UClass::StaticClass();
	const bool bIsActor = bIsClass ? CastChecked<UClass>(Item->AssetData.GetAsset())->IsChildOf(AActor::StaticClass()) : false;

	AActor* DefaultActor = nullptr;
	if (Item->Factory != nullptr)
	{
		DefaultActor = Item->Factory->GetDefaultActor(Item->AssetData);
	}
	else if (bIsActor)
	{
		DefaultActor = CastChecked<AActor>(CastChecked<UClass>(Item->AssetData.GetAsset())->GetDefaultObject(false));
	}

	UClass* DocClass = nullptr;
	TSharedPtr<IToolTip> AssetEntryToolTip;
	if(DefaultActor != nullptr)
	{
		DocClass = DefaultActor->GetClass();
		AssetEntryToolTip = FEditorClassUtils::GetTooltip(DefaultActor->GetClass());
	}

	if (!AssetEntryToolTip.IsValid())
	{
		AssetEntryToolTip = FSlateApplicationBase::Get().MakeToolTip(Item->DisplayName);
	}
	
	const FButtonStyle& ButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>( "Menu.Button" );
	const float MenuIconSize = FAppStyle::Get().GetFloat("Menu.MenuIconSize");

	Style = &ButtonStyle;

	// Create doc link widget if there is a class to link to
	TSharedRef<SWidget> DocWidget = SNew(SSpacer);
	if(DocClass != NULL)
	{
		DocWidget = FEditorClassUtils::GetDocumentationLinkWidget(DocClass);
		DocWidget->SetCursor( EMouseCursor::Default );
	}

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SNew(SBorder)
		.BorderImage( this, &SObjectMixerPlacementAssetMenuEntry::GetBorder )
		.Cursor( EMouseCursor::GrabHand )
		.ToolTip( AssetEntryToolTip )
		.Padding(FMargin(5.f, 3.f, 5.f, 3.f))
		[
			SNew( SHorizontalBox )

			+ SHorizontalBox::Slot()
			.Padding(10.0f, 0.f, 10.f, 0.0f)
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(MenuIconSize)
				.HeightOverride(MenuIconSize)
				[
					SNew(SImage)
					.Image(this, &SObjectMixerPlacementAssetMenuEntry::GetIcon)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(1.f, 0.f, 0.f, 0.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[

				SNew( STextBlock )
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Text( Item->DisplayName )
			]


			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SImage)
                .ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.DragHandle"))
			]
		]
	];
}


const FSlateBrush* SObjectMixerPlacementAssetMenuEntry::GetIcon() const
{

	if (AssetImage != nullptr)
	{
		return AssetImage;
	}

	if (Item->ClassIconBrushOverride != NAME_None)
	{
		AssetImage = FSlateIconFinder::FindCustomIconBrushForClass(nullptr, TEXT("ClassIcon"), Item->ClassIconBrushOverride);
	}
	else
	{
		AssetImage = FSlateIconFinder::FindIconBrushForClass(FClassIconFinder::GetIconClassForAssetData(Item->AssetData));
	}

	return AssetImage;
}


FReply SObjectMixerPlacementAssetMenuEntry::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = true;

		return FReply::Handled().DetectDrag( SharedThis( this ), MouseEvent.GetEffectingButton() );
	}

	return FReply::Unhandled();
}

FReply SObjectMixerPlacementAssetMenuEntry::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = false;

		AActor* NewActor = nullptr;
		UActorFactory* Factory = Item->Factory;
		if (!Item->Factory)
		{
			// If no actor factory was found or failed, add the actor from the uclass
			UClass* AssetClass = Item->AssetData.GetClass();
			if (AssetClass)
			{
				UObject* ClassObject = AssetClass->GetDefaultObject();
				FActorFactoryAssetProxy::GetFactoryForAssetObject(ClassObject);
			}
		}
		NewActor = FLevelEditorActionCallbacks::AddActor(Factory, Item->AssetData, nullptr);
		if (NewActor && GCurrentLevelEditingViewportClient)
		{
  			GEditor->MoveActorInFrontOfCamera(*NewActor, 
  				GCurrentLevelEditingViewportClient->GetViewLocation(), 
  				GCurrentLevelEditingViewportClient->GetViewRotation().Vector()
  			);
		}

		if (!MouseEvent.IsControlDown())
		{
			FSlateApplication::Get().DismissAllMenus();
		}

		GEditor->SelectNone(true, true, false);
		GEditor->SelectActor(NewActor, true, true);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SObjectMixerPlacementAssetMenuEntry::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bIsPressed = false;

	if (FEditorDelegates::OnAssetDragStarted.IsBound())
	{
		TArray<FAssetData> DraggedAssetDatas;
		DraggedAssetDatas.Add( Item->AssetData );
		FEditorDelegates::OnAssetDragStarted.Broadcast( DraggedAssetDatas, Item->Factory );
		return FReply::Handled();
	}

	if( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) )
	{
		return FReply::Handled().BeginDragDrop(FAssetDragDropOp::New(Item->AssetData, Item->Factory));
	}
	else
	{
		return FReply::Handled();
	}
}

bool SObjectMixerPlacementAssetMenuEntry::IsPressed() const
{
	return bIsPressed;
}

const FSlateBrush* SObjectMixerPlacementAssetMenuEntry::GetBorder() const
{
	if ( IsPressed() )
	{
		return &(Style->Pressed);
	}
	else if ( IsHovered() )
	{
		return &(Style->Hovered);
	}
	else
	{
		return &(Style->Normal);
	}
}

FSlateColor SObjectMixerPlacementAssetMenuEntry::GetForegroundColor() const
{
	if (IsPressed())
	{
		return Style->PressedForeground;
	}
	else if (IsHovered())
	{
		return Style->HoveredForeground;
	}
	else
	{
		return Style->NormalForeground;
	}
}
