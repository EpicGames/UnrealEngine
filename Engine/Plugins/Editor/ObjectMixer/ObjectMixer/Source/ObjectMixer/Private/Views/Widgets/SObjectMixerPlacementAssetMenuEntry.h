// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

struct FButtonStyle;
struct FPlaceableItem;

class SObjectMixerPlacementAssetMenuEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SObjectMixerPlacementAssetMenuEntry){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<const struct FPlaceableItem>& InItem);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	bool IsPressed() const;

	TSharedPtr<const FPlaceableItem> Item;

	virtual FSlateColor GetForegroundColor() const override;

private:
	const FSlateBrush* GetBorder() const;
	const FSlateBrush* GetIcon() const;

	bool bIsPressed = false;

	const FButtonStyle* Style = nullptr;
	
	mutable const FSlateBrush* AssetImage = nullptr;
};
