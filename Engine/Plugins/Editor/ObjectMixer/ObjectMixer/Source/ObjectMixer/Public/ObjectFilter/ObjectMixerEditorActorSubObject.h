// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "UObject/WeakObjectPtr.h"

struct FSlateBrush;

/** Container property mixer entry for displaying a container property (e.g. Array) in the mixer */
struct FObjectMixerEditorContainerProperty
{
	/**
	 * If the subobject is stored in a container, supplying the container property's name will allow the mixer to display
	 * a row item for the container property itself
	 */
	FName PropertyName = NAME_None;

	/** Optional custom icon to display for the container property; if not provided, a default one will be used */
	TAttribute<const FSlateBrush*> PropertyIconOverride;
};

/**
 * Actor sub-object mixer entry that controls how the sub-object is displayed in the mixer
 */
struct FObjectMixerEditorActorSubObject
{
	/** Pointer to the subobject */
	TWeakObjectPtr<UObject> Object = nullptr;

	/** Optional display name to use when displaying the object in the mixer */
	FText DisplayName = FText::GetEmpty();

	/** Whether to include all parent objects up to the owning actor in the mixer */
	bool bIncludeObjectStack = false;

	/** Optional container property to make the sub-object a child of in the mixer */
	FObjectMixerEditorContainerProperty ContainerProperty;
};