// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectMixerEditorActorSubObject.h"
#include "GameFramework/Actor.h"
#include "SceneOutlinerDragDrop.h"
#include "UObject/Class.h"

#include "ObjectMixerEditorObjectFilter.generated.h"

#define UE_API OBJECTMIXEREDITOR_API

struct FToolMenuContext;

UENUM(BlueprintType)
enum class EObjectMixerInheritanceInclusionOptions : uint8
{
	// Get only the properties in the specified classes without considering parent or child classes
	None,
	// Get properties from the class that the specified classes immediately derive from, but not the parents' parents + Specified Classes
	IncludeOnlyImmediateParent,
	// Get properties from child classes but not child classes of child classes + Specified Classes
	IncludeOnlyImmediateChildren,
	// IncludeOnlyImmediateParent + IncludeOnlyImmediateChildren + Specified Classes
	IncludeOnlyImmediateParentAndChildren,
	// Go up the chain of parent classes to get all properties in the specified classes' ancestries + Specified Classes
	IncludeAllParents,
	// Get properties from all derived classes recursively + Specified Classes
	IncludeAllChildren,
	// IncludeAllParents + IncludeAllChildren + Specified Classes
	IncludeAllParentsAndChildren,
	// IncludeAllParents + IncludeOnlyImmediateChildren + Specified Classes
	IncludeAllParentsAndOnlyImmediateChildren, 
	// IncludeOnlyImmediateParent + IncludeAllChildren + Specified Classes
	IncludeOnlyImmediateParentAndAllChildren 
};

UENUM(BlueprintType)
enum class EObjectMixerTreeViewMode : uint8
{
	// Show all matching objects without folders
	NoFolders,
	// Display objects in a hierarchy with folders
	Folders 
};

/**
 * Native class for filtering object types to Object Mixer.
 * Native C++ classes should inherit directly from this class.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UObjectMixerObjectFilter : public UObject
{
	GENERATED_BODY()
	
public:

	UObjectMixerObjectFilter() = default;

	/** Begin UObject overrides */
	UE_API virtual void PostCDOCompiled(const FPostCDOCompiledContext& Context) override;
	/** End UObject overrides */
	
	/**
	 * Return the basic object types you want to filter for in your level.
	 * For example, if you want to work with Lights, return ULightComponentBase.
	 * If you also want to see the properties for parent or child classes,
	 * override the GetObjectMixerPropertyInheritanceInclusionOptions and GetForceAddedColumns functions.
	 */
	virtual TSet<UClass*> GetObjectClassesToFilter() const { return {}; }

	/**
	 * Return the basic actor types you want to be able to place using the Add button.
	 * Note that only subclasses of AActor are supported and only those which have a registered factory.
	 * This includes most engine actor types.
	 */
	virtual TSet<TSubclassOf<AActor>> GetObjectClassesToPlace() const { return {}; }

	/**
	 * Determines if transient objects (such as Sequencer Spawnables) should be shown in the list. False by default.
	 */
	UE_API virtual bool GetShowTransientObjects() const;

	/**
	 * Determines if hybrid rows should be used when enabled in user settings. True by default.
	 */
	UE_API virtual bool ShouldAllowHybridRows() const;

	/**
	 * Determines if the user should be able to change which columns are displayed. True by default.
	 */
	UE_API virtual bool ShouldAllowColumnCustomizationByUser() const;

	/**
	 * Specify a list of property names corresponding to columns you want to show by default.
	 * For example, you can specify "Intensity" and "LightColor" to show only those property columns by default in the UI.
	 * Columns not specified will not be shown by default but can be enabled by the user in the UI.
	 */
	UE_API virtual TSet<FName> GetColumnsToShowByDefault() const;

	/**
	 * Specify a list of property names corresponding to columns you don't want to ever show.
	 * For example, you can specify "Intensity" and "LightColor" to ensure that they can't be enabled or shown in the UI.
	 * Columns not specified can be enabled by the user in the UI.
	 */
	UE_API virtual TSet<FName> GetColumnsToExclude() const;

	/**
	 * Specify a list of property names found in parent classes you want to show that aren't in the specified classes.
	 * Note that properties specified here do not override the properties specified in GetColumnsToExclude(),
	 * but do override the supported property tests so these will appear even if ShouldIncludeUnsupportedProperties returns false.
	 * For example, a ULightComponent displays "LightColor" in the editor's details panel,
	 * but ULightComponent itself doesn't have a property named "LightColor". Instead it's in its parent class, ULightComponentBase.
	 * In this scenario, ULightComponent is specified and PropertyInheritanceInclusionOptions is None, so "LightColor" won't appear by default.
	 * Specify "LightColor" in this function to ensure that "LightColor" will appear as a column as long as
	 * the property is accessible to one of the specified classes regardless of which parent class it comes from.
	 */
	virtual TSet<FName> GetForceAddedColumns() const { return {}; }

	/**
	 * Specify whether we should return only the properties of the specified classes or the properties of parent and child classes.
	 * Defaults to 'None' which only considers the properties of the specified classes.
	 * If you're not seeing all the properties you expected, try overloading this function.
	 */
	UE_API virtual EObjectMixerInheritanceInclusionOptions GetObjectMixerPropertyInheritanceInclusionOptions() const;

	/**
	 * Specify whether we should return only the specified classes or the parent and child classes in placement mode.
	 * Defaults to 'None' which only considers the specified classes.
	 */
	UE_API virtual EObjectMixerInheritanceInclusionOptions GetObjectMixerPlacementClassInclusionOptions() const;

	/**
	 * If true, properties that are not visible in the details panel and properties not supported by SSingleProperty will be selectable.
	 * Defaults to false.
	 */
	UE_API virtual bool ShouldIncludeUnsupportedProperties() const;

	/**
	 * If a property is changed that has a name found in this set, the panel will be refreshed.
	 * Add a property name to this list if you expect the list to change in some way after changing that property.
	 */
	UE_API virtual TSet<FName> GetPropertiesThatRequireListRefresh() const;

	/**
	 * Fills OutAssociatedActors with additional actors to include as children in the ObjectMixer hierarchy for the given actor.
	 */
	UE_API virtual TArray<AActor*> FindAssociatedActors(AActor* InActor) const;

	/**
	 * Returns true if AssociatedActor is associated with Actor.
	 * This should return true if the actor would be included in FindAssociatedActors, but is a separate function to avoid allocating an array every time.
	 */
	UE_API virtual bool IsActorAssociated(AActor* Actor, AActor* AssociatedActor) const;

	/**
	 * Gets a list of subobjects to add to the outliner under the specified actor
	 */
	UE_API virtual TArray<FObjectMixerEditorActorSubObject> GetActorSubObjects(AActor* InActor) const;
	
	/** 
	 * Returns true if the given tree item should override the Object Mixer's standard handling for drop operations (ValidateDrop/OnDrop)
	 */
	UE_API virtual bool HasCustomDropHandling(const ISceneOutlinerTreeItem& DropTarget) const;

	/** Test whether the specified payload can be dropped onto a tree item */
	UE_API virtual FSceneOutlinerDragValidationInfo ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const;

	/** Called when a payload is dropped onto a target */
	UE_API virtual void OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const;

	/** Called when a context menu is being created from in the hierarchy. Allows subclasses to inject additonal context data. */
	virtual void OnContextMenuContextCreated(FToolMenuContext& Context) const {}

	static UE_API TSet<UClass*> GetParentAndChildClassesFromSpecifiedClasses(
		const TSet<UClass*>& InSpecifiedClasses, EObjectMixerInheritanceInclusionOptions Options);

	static UE_API TSet<UClass*> GetParentAndChildClassesFromSpecifiedClasses(
		const TSet<TSubclassOf<AActor>>& InSpecifiedClasses, EObjectMixerInheritanceInclusionOptions Options);
};

/**
 * Script class for filtering object types to Object Mixer.
 * Blueprint classes should inherit directly from this class.
 */
UCLASS(MinimalAPI, Abstract, Blueprintable)
class UObjectMixerBlueprintObjectFilter : public UObjectMixerObjectFilter
{
	GENERATED_BODY()
public:
	
	/**
	 * Return the basic object types you want to filter for in your level.
	 * For example, if you want to work with Lights, return ULightComponentBase.
	 * If you also want to see the properties for parent or child classes,
	 * override the GetObjectMixerPropertyInheritanceInclusionOptions and GetForceAddedColumns functions.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Object Mixer")
	UE_API TSet<UClass*> GetObjectClassesToFilter() const override;

	TSet<UClass*> GetObjectClassesToFilter_Implementation() const
	{
		return Super::GetObjectClassesToFilter();
	}

	/**
	 * Return the basic actor types you want to be able to place using the Add button.
	 * Note that only subclasses of AActor are supported and only those which have a registered factory.
	 * This includes most engine actor types.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Object Mixer")
	UE_API TSet<TSubclassOf<AActor>> GetObjectClassesToPlace() const override;

	TSet<TSubclassOf<AActor>> GetObjectClassesToPlace_Implementation() const
	{
		return Super::GetObjectClassesToPlace();
	}

	/**
	 * Determines if transient objects (such as Sequencer Spawnables) should be shown in the list. False by default.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Object Mixer")
	UE_API bool GetShowTransientObjects() const;

	bool GetShowTransientObjects_Implementation() const
	{
		return Super::GetShowTransientObjects();
	}
	
	/**
	 * Specify a list of property names corresponding to columns you want to show by default.
	 * For example, you can specify "Intensity" and "LightColor" to show only those property columns by default in the UI.
	 * Columns not specified will not be shown by default but can be enabled by the user in the UI.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Object Mixer")
	UE_API TSet<FName> GetColumnsToShowByDefault() const override;
	
	TSet<FName> GetColumnsToShowByDefault_Implementation() const
	{
		return Super::GetColumnsToShowByDefault();
	}
	
	/**
	 * Specify a list of property names corresponding to columns you don't want to ever show.
	 * For example, you can specify "Intensity" and "LightColor" to ensure that they can't be enabled or shown in the UI.
	 * Columns not specified can be enabled by the user in the UI.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Object Mixer")
	UE_API TSet<FName> GetColumnsToExclude() const override;

	TSet<FName> GetColumnsToExclude_Implementation() const
	{
		return Super::GetColumnsToExclude();
	}

	/**
	 * Specify a list of property names found in parent classes you want to show that aren't in the specified classes.
	 * Note that properties specified here do not override the properties specified in GetColumnsToExclude().
	 * For example, a ULightComponent displays "LightColor" in the editor's details panel,
	 * but ULightComponent itself doesn't have a property named "LightColor". Instead it's in its parent class, ULightComponentBase.
	 * In this scenario, ULightComponent is specified and PropertyInheritanceInclusionOptions is None, so "LightColor" won't appear by default.
	 * Specify "LightColor" in this function to ensure that "LightColor" will appear as a column as long as
	 * the property is accessible to one of the specified classes regardless of which parent class it comes from.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Object Mixer")
	UE_API TSet<FName> GetForceAddedColumns() const override;
	
	TSet<FName> GetForceAddedColumns_Implementation() const
	{
		return Super::GetForceAddedColumns();
	}
	
	/**
	 * Specify whether we should return only the properties of the specified classes or the properties of parent and child classes.
	 * Defaults to 'None' which only considers the properties of the specified classes.
	 * If you're not seeing all the properties you expected, try overloading this function.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Object Mixer")
	UE_API EObjectMixerInheritanceInclusionOptions GetObjectMixerPropertyInheritanceInclusionOptions() const override;

	EObjectMixerInheritanceInclusionOptions GetObjectMixerPropertyInheritanceInclusionOptions_Implementation() const
	{
		return Super::GetObjectMixerPropertyInheritanceInclusionOptions();
	}

	/**
	 * Specify whether we should return only the specified classes or the parent and child classes in placement mode.
	 * Defaults to 'None' which only considers the specified classes.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Object Mixer")
	UE_API EObjectMixerInheritanceInclusionOptions GetObjectMixerPlacementClassInclusionOptions() const override;

	EObjectMixerInheritanceInclusionOptions GetObjectMixerPlacementClassInclusionOptions_Implementation() const
	{
		return Super::GetObjectMixerPlacementClassInclusionOptions();
	}

	/**
	 * If true, properties that are not visible in the details panel and properties not supported by SSingleProperty will be selectable.
	 * Defaults to false.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Object Mixer")
	UE_API bool ShouldIncludeUnsupportedProperties() const override;

	bool ShouldIncludeUnsupportedProperties_Implementation() const
	{
		return Super::ShouldIncludeUnsupportedProperties();
	}
	
	/**
	 * If a property is changed that has a name found in this set, the panel will be refreshed.
	 * Add a property name to this list if you expect the list to change in some way after changing that property.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Object Mixer")
	UE_API TSet<FName> GetPropertiesThatRequireListRefresh() const override;

	TSet<FName> GetPropertiesThatRequireListRefresh_Implementation() const
	{
		return Super::GetPropertiesThatRequireListRefresh();
	}

};

#undef UE_API
