// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_CLASS(LogObjectMixerEditor, Log, All);
