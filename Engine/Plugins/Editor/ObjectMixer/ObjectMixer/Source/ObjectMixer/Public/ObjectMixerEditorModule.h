// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"
#include "Widgets/Docking/SDockTab.h"

#define UE_API OBJECTMIXEREDITOR_API

class FObjectMixerEditorList;
class ISequencer;

class FObjectMixerEditorModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface Interface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	//~ End IModuleInterface

	UE_API virtual UWorld* GetWorld();

	UE_API virtual void Initialize();
	UE_API virtual void Teardown();

	static UE_API FObjectMixerEditorModule& Get();

	static UE_API void OpenProjectSettings();

	UE_API virtual FName GetModuleName() const;
	
	UE_API virtual TSharedPtr<SWidget> MakeObjectMixerDialog(
		TSubclassOf<UObjectMixerObjectFilter> InDefaultFilterClass = {});

	/** Get a list of sequencers that are currently active in the editor. */
	UE_API virtual TArray<TWeakPtr<ISequencer>> GetSequencers() const;

	/**
	 * Tries to find the nomad tab assigned to this instance of Object Mixer.
	 * If DockTab is not set, will try to find the tab using GetTabSpawnerId().
	 */
	UE_API virtual TSharedPtr<SDockTab> FindNomadTab();
	
	/**
	 * Build the List widget from scratch. If DockTab is not set, will try to find the tab using GetTabSpawnerId().
	 * @return True if the widget was regenerated. False if the DockTab was invalid and could not be found.
	 */
	UE_API bool RegenerateListWidget();

	/** Called when the Rename command is executed from the UI or hotkey. */
	UE_API virtual void OnRenameCommand();
	
	UE_API void RegisterMenuGroup();
	UE_API void UnregisterMenuGroup();
	UE_API virtual void SetupMenuItemVariables();
	UE_API virtual void RegisterTabSpawner();
	UE_API virtual FName GetTabSpawnerId();
	
	/**
	 * Add a tab spawner to the Object Mixer menu group.
	 * @return If adding the item to the menu was successful
	 */
	UE_API bool RegisterItemInMenuGroup(FWorkspaceItem& InItem);
	
	UE_API virtual void UnregisterTabSpawner();
	UE_API virtual void RegisterSettings() const;
	UE_API virtual void UnregisterSettings() const;

	UE_API virtual TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

	UE_API TSharedPtr<FWorkspaceItem> GetWorkspaceGroup();

	/**
	 * This is the filter class used to initialize the ListModel.
	 * This filter class cannot be turned off by the end user.
	 */
	UE_API const TSubclassOf<UObjectMixerObjectFilter>& GetDefaultFilterClass() const;

	UE_API const static FName BaseObjectMixerModuleName;
	
	DECLARE_MULTICAST_DELEGATE(FOnBlueprintFilterCompiled);
	FOnBlueprintFilterCompiled& OnBlueprintFilterCompiled()
	{
		return OnBlueprintFilterCompiledDelegate;
	}

protected:

	/** Lives for as long as the module is loaded. */
	TSharedPtr<FObjectMixerEditorList> ListModel;

	/** The text that appears on the spawned nomad tab */
	FText TabLabel;

	/** The actual spawned nomad tab */
	TWeakPtr<SDockTab> DockTab;

	/** Menu Item variables */
	FText MenuItemName;
	FSlateIcon MenuItemIcon;
	FText MenuItemTooltip;
	ETabSpawnerMenuType::Type TabSpawnerType = ETabSpawnerMenuType::Enabled;

	/**
	 * If set, this is the filter class used to initialize the ListModel.
	 * This filter class cannot be turned off by the end user.
	 */
	TSubclassOf<UObjectMixerObjectFilter> DefaultFilterClass;

	FOnBlueprintFilterCompiled OnBlueprintFilterCompiledDelegate;

private:

	TSharedPtr<FWorkspaceItem> WorkspaceGroup;
};

#undef UE_API
