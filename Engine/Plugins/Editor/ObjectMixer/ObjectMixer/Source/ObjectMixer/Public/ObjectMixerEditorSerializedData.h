// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectMixerEditorSerializedData.generated.h"

#define UE_API OBJECTMIXEREDITOR_API

DECLARE_MULTICAST_DELEGATE(FOnObjectMixerCollectionChanged)
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnObjectMixerCollectionRenamed, FName, FName)

USTRUCT()
struct FObjectMixerCollectionObjectData
{
	GENERATED_BODY()

	UPROPERTY()
	FSoftObjectPath ObjectPath = {};

	bool operator==(const FObjectMixerCollectionObjectData& Other) const
	{
		return ObjectPath == Other.ObjectPath;
	}

	friend uint32 GetTypeHash (const FObjectMixerCollectionObjectData& Other)
	{
		return GetTypeHash(Other.ObjectPath);
	}

};

USTRUCT()
struct FObjectMixerCollectionObjectSet
{
	GENERATED_BODY()
	
	UPROPERTY()
	FName CollectionName = NAME_None;

	UPROPERTY()
	TArray<FObjectMixerCollectionObjectData> CollectionObjects = {};

	bool operator==(const FObjectMixerCollectionObjectSet& Other) const
	{
		return CollectionName.IsEqual(Other.CollectionName);
	}

	friend uint32 GetTypeHash (const FObjectMixerCollectionObjectSet& Other)
	{
		return GetTypeHash(Other.CollectionName);
	}
};

USTRUCT()
struct FObjectMixerColumnData
{
	GENERATED_BODY()
	
	UPROPERTY()
	FName ColumnName = NAME_None;

	UPROPERTY()
	bool bShouldBeEnabled = false;

	bool operator==(const FObjectMixerColumnData& Other) const
	{
		return ColumnName.IsEqual(Other.ColumnName);
	}

	friend uint32 GetTypeHash (const FObjectMixerColumnData& Other)
	{
		return GetTypeHash(Other.ColumnName);
	}
};

USTRUCT()
struct FObjectMixerSerializationDataPerFilter
{
	GENERATED_BODY()
	
	UPROPERTY()
	FName FilterClassName = NAME_None;

	UPROPERTY()
	TArray<FObjectMixerCollectionObjectSet> SerializedCollections = {};

	UPROPERTY()
	TSet<FObjectMixerColumnData> SerializedColumnData = {};

	bool operator==(const FObjectMixerSerializationDataPerFilter& Other) const
	{
		return FilterClassName.IsEqual(Other.FilterClassName);
	}

	friend uint32 GetTypeHash (const FObjectMixerSerializationDataPerFilter& Other)
	{
		return GetTypeHash(Other.FilterClassName);
	}
};

UCLASS(MinimalAPI, config = ObjectMixerSerializedData)
class UObjectMixerEditorSerializedData : public UObject
{
	GENERATED_BODY()
public:
	
	UObjectMixerEditorSerializedData(const FObjectInitializer& ObjectInitializer)
	{
		// Makes undo/redo possible for this object
		SetFlags(RF_Transactional);
	}
	
	UPROPERTY(Config)
	TSet<FObjectMixerSerializationDataPerFilter> SerializedDataPerFilter;

	UE_API FObjectMixerSerializationDataPerFilter* FindSerializationDataByFilterClassName(const FName& FilterClassName);

	UE_API bool AddObjectsToCollection(const FName& FilterClassName, const FName& CollectionName, const TSet<FSoftObjectPath>& ObjectsToAdd);

	UE_API bool RemoveObjectsFromCollection(const FName& FilterClassName, const FName& CollectionName, const TSet<FSoftObjectPath>& ObjectsToRemove);

	UE_API bool RemoveCollection(const FName& FilterClassName, const FName& CollectionName);
	
	UE_API bool DuplicateCollection(const FName& FilterClassName, const FName& CollectionToDuplicateName, FName& DesiredDuplicateName);

	UE_API bool ReorderCollection(const FName& FilterClassName, const FName& CollectionToMoveName, const FName& CollectionInsertBeforeName);

	UE_API bool RenameCollection(const FName& FilterClassName, const FName CollectionNameToRename, const FName NewCollectionName);

	UE_API bool DoesCollectionExist(const FName& FilterClassName, const FName& CollectionName);

	UE_API bool IsObjectInCollection(const FName& FilterClassName, const FName& CollectionName, const FSoftObjectPath& InObject);

	UE_API TSet<FName> GetCollectionsForObject(const FName& FilterClassName, const FSoftObjectPath& InObject);

	UE_API TArray<FName> GetAllCollectionNames(const FName& FilterClassName);

	UE_API void SetShouldShowColumn(const FName& FilterClassName, const FName& ColumnName, const bool bNewShouldShowColumn);

	UE_API bool IsColumnDataSerialized(const FName& FilterClassName, const FName& ColumnName);
	
	UE_API bool ShouldShowColumn(const FName& FilterClassName, const FName& ColumnName);

	static UE_API FName AllCollectionName;

	FOnObjectMixerCollectionChanged OnObjectMixerCollectionMapChanged;
	FOnObjectMixerCollectionRenamed OnObjectMixerCollectionRenamed;
};

#undef UE_API
