// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "ObjectMixerEditorSettings.generated.h"

UENUM(BlueprintType)
enum class EObjectMixerHybridMode : uint8
{
	HybridActor UMETA(DisplayName="Hybrid Rows (Select Actor)", ToolTip="Makes hybrid rows and selects the actor in the Scene Outliner when an actor with a single matching component is selected."),
	HybridComponent UMETA(DisplayName="Hybrid Rows (Select Component)", ToolTip="Makes hybrid rows and selects the component in the Scene Outliner when an actor with a single matching component is selected."),
	HybridNone UMETA(DisplayName="Do not make hybrid rows", ToolTip="Does not make hybrid rows when an actor with a single matching component is selected. Actor and component rows will be split."),
};

UCLASS(MinimalAPI, config = ObjectMixer)
class UObjectMixerEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	
	static bool DoesPropertyChangeRequireListRebuild(const FPropertyChangedEvent& Event)
	{
		return Event.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UObjectMixerEditorSettings, HybridRowPolicy);
	}

	bool IsHybridRowModeEnabled() const
	{
		return HybridRowPolicy != EObjectMixerHybridMode::HybridNone;
	}
	
	/**
	 * If enabled, clicking an item in the mixer list will also select the item in the Scene Outliner.
	 * Alt + Click to select items in mixer without selecting the item in the Scene outliner.
	 * If disabled, selections will not sync unless Alt is held. Effectively, this is the opposite behavior.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Object Mixer")
	bool bSyncSelection = true;

	/**
	 * Determines how rows with a single matching component are displayed. By default, these rows are condensed into a single row.
	 * Selecting that row will select the actor in the scene outliner, but not the component. You can choose to select the component instead,
	 * or choose to not condense the rows and leave them separated.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Object Mixer")
	EObjectMixerHybridMode HybridRowPolicy = EObjectMixerHybridMode::HybridActor;
};
