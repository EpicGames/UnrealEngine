// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

#define UE_API OBJECTMIXEREDITOR_API

class FObjectMixerEditorStyle : FSlateStyleSet
{
public:

	static UE_API void Initialize();

	static UE_API void Shutdown();

	static UE_API void ReloadTextures();

	static UE_API const ISlateStyle& Get();

	UE_API virtual const FName& GetStyleSetName() const override;

	UE_API virtual const FSlateBrush* GetBrush(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const ISlateStyle* RequestingStyle = nullptr) const override;

	template< typename WidgetStyleType >
	static const WidgetStyleType& GetWidgetStyle(FName PropertyName, const ANSICHAR* Specifier = nullptr)
	{
		return StyleInstance->GetWidgetStyle<WidgetStyleType>(PropertyName, Specifier);
	}

private:

	static UE_API FString GetExternalPluginContent(const FString& PluginName, const FString& RelativePath, const ANSICHAR* Extension);
	
	static UE_API TSharedRef<FSlateStyleSet> Create();

	static UE_API TSharedPtr<FSlateStyleSet> StyleInstance;
};

#undef UE_API
