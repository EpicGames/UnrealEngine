// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "ObjectMixerFilterFactory.generated.h"

#define UE_API OBJECTMIXEREDITOR_API

UCLASS(MinimalAPI, hidecategories = Object)
class UObjectMixerBlueprintFilterFactory : public UFactory
{
	GENERATED_BODY()
public:
	UE_API UObjectMixerBlueprintFilterFactory();
	
	// The parent class of the created blueprint
    UPROPERTY(VisibleAnywhere, Category="ObjectMixerBlueprintFilterFactory", meta=(AllowAbstract = "", BlueprintBaseOnly = ""))
    TSubclassOf<UObject> ParentClass;
	
	//~ Begin UFactory Interface
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual FString GetDefaultNewAssetName() const override;
	UE_API virtual uint32 GetMenuCategories() const override;
	UE_API virtual FText GetToolTip() const override;
	UE_API virtual FString GetToolTipDocumentationExcerpt() const override;
	UE_API virtual TArray<FAssetCategoryPath> GetAssetMenuPathsForCategory(FName InCategory) const override;
	//~ Begin UFactory Interface
	
};

#undef UE_API
