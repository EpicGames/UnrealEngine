// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class AActor;
class UActorComponent;

/**
 * Provides an interface for synchronizing an ObjectMixer's selection with another part of the editor
 */
class IObjectMixerSelectionInterface : public TSharedFromThis<IObjectMixerSelectionInterface>
{
public:
	DECLARE_MULTICAST_DELEGATE(FOnSelectionChanged);

public:
	virtual ~IObjectMixerSelectionInterface() = default;

	/**
	 * Select or deselect a list of components.
	 * @param	InSelectedActors		The actors to select.
	 * @param	bShouldSelect			If true, select the actors and deselect all other actors. If false, only deselect the listed actors.
	 * @param	bSelectEvenIfHidden		If false, only select actors that are visible (e.g. not filtered out) in the editor.
	 */
	virtual void SelectActors(const TArray<AActor*>& InSelectedActors, bool bShouldSelect, bool bSelectEvenIfHidden) = 0;

	/**
	 * Select or deselect a list of components.
	 * @param	InSelectedComponents	The components to select.
	 * @param	bShouldSelect			If true, select the components and deselect all other components. If false, only deselect the listed components.
	 * @param	bSelectEvenIfHidden		If false, only select components that are visible (e.g. not filtered out) in the editor.
	 */
	virtual void SelectComponents(const TArray<UActorComponent*>& InSelectedComponents, bool bShouldSelect, bool bSelectEvenIfHidden) = 0;

	/**
	 * Select or deselect a list of actor sub-objects.
	 * @param	InSelectedSubObjects	The sub-objects to select.
	 * @param	bShouldSelect			If true, select the sub-objects and deselect all other sub-objects. If false, only deselect the listed sub-objects.
	 * @param	bSelectEvenIfHidden		If false, only select sub-objects that are visible (e.g. not filtered out) in the editor.
	 */
	virtual void SelectSubObjects(const TArray<UObject*>& InSelectedSubObjects, bool bShouldSelect, bool bSelectEvenIfHidden) = 0;
	
	/** Get the list of selected actors. */
	virtual TArray<AActor*> GetSelectedActors() const = 0;

	/** Get the list of selected components. */
	virtual TArray<UActorComponent*> GetSelectedComponents() const = 0;

	/** Get the list of selected sub-objects */
	virtual TArray<UObject*> GetSelectedSubObjects() const = 0;
	
	/** Get the event handler that is raised when the synchronized selection changes. */
	virtual FOnSelectionChanged& OnSelectionChanged() = 0;
};
