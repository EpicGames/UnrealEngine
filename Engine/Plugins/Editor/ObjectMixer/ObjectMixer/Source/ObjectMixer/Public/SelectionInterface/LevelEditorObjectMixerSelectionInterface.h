// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IObjectMixerSelectionInterface.h"

#define UE_API OBJECTMIXEREDITOR_API

/**
 * Provides an interface for the ObjectMixer to synchronize with the level editor's selections (via GEditor)
 */
class FLevelEditorObjectMixerSelectionInterface : public IObjectMixerSelectionInterface
{
public:

	UE_API FLevelEditorObjectMixerSelectionInterface();
	UE_API virtual ~FLevelEditorObjectMixerSelectionInterface() override;

	//~ Begin IObjectMixerSelectionInterface interface
	UE_API virtual void SelectActors(const TArray<AActor*>& InSelectedActors, bool bShouldSelect, bool bSelectEvenIfHidden) override;
	UE_API virtual void SelectComponents(const TArray<UActorComponent*>& InSelectedComponents, bool bShouldSelect, bool bSelectEvenIfHidden) override;
	virtual void SelectSubObjects(const TArray<UObject*>& InSelectedSubObjects, bool bShouldSelect, bool bSelectEvenIfHidden) override { }
	UE_API virtual TArray<AActor*> GetSelectedActors() const override;
	UE_API virtual TArray<UActorComponent*> GetSelectedComponents() const override;
	virtual TArray<UObject*> GetSelectedSubObjects() const override { return { }; }
	virtual FOnSelectionChanged& OnSelectionChanged() override { return SelectionChanged; }
	//~ End IObjectMixerSelectionInterface interface

private:
	UE_API void OnLevelSelectionChanged(UObject* Obj);

	FOnSelectionChanged SelectionChanged;
};

#undef UE_API
