// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerHierarchy.h"

class FObjectMixerOutlinerMode;
class FWorldPartitionActorDescInstance;
class UActorFolder;
class ULevelInstanceSubsystem;
class UObjectMixerObjectFilter;
class UWorldPartition;

class FObjectMixerOutlinerHierarchy : public ISceneOutlinerHierarchy, public FNoncopyable
{
public:
	virtual ~FObjectMixerOutlinerHierarchy();

	static TUniquePtr<FObjectMixerOutlinerHierarchy> Create(class ISceneOutlinerMode* Mode, const TWeakObjectPtr<UWorld>& World);

	FObjectMixerOutlinerMode* GetCastedMode() const;
	const UObjectMixerObjectFilter* GetObjectFilter() const;

	/** Create a linearization of all applicable items in the hierarchy */
	virtual void CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
	/** Create a linearization of all direct and indirect children of a given item in the hierarchy */
	virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const override;
	/** Forcibly create a parent item for a given tree item */
	virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) override;

	void SetShowingComponents(bool bInShowingComponents) { bShowingComponents = bInShowingComponents; }
	void SetShowingOnlyActorWithValidComponents(bool bInShowingOnlyActorWithValidComponents) { bShowingOnlyActorWithValidComponents = bInShowingOnlyActorWithValidComponents; }
	void SetShowingLevelInstances(bool bInShowingLevelInstances) { bShowingLevelInstances = bInShowingLevelInstances; }
	void SetShowingUnloadedActors(bool bInShowingUnloadedActors) { bShowingUnloadedActors = bInShowingUnloadedActors; }
	void SetShowingEmptyFolders(bool bInShowingEmptyFolders) { bShowingEmptyFolders = bInShowingEmptyFolders; }

protected:

	bool DoesWorldObjectHaveAcceptableClass(const UObject* Object) const;

	void CreateFolderChild(const FFolder& Folder, UWorld* World, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;

	void ForEachActorInLevel(AActor* Actor, const ULevelInstanceSubsystem* LevelInstanceSubsystem, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;
	
	/** Adds all the direct and indirect children of a world to OutItems */
	void CreateWorldChildren(UWorld* World, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;
	
	/**
	 * Create all component items for an actor if we are showing components and place them in OutItems. Items not added if filtered out.
	 * @return true if any items would have been returned before filtering
	 */
	bool CreateComponentItems(AActor* Actor, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;

	/**
	 * Create all associated actor items for an actor and place them in OutItems. Items not added if filtered out.
	 * @return true if any items would have been returned before filtering
	 */
	bool CreateAssociatedActorItems(AActor* Actor, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;

	/**
	 * Create all subobject items for an actor and place them in OutItems.
	 * @param Actor The actor to get subobjects for
	 * @param OutItems The resulting list of tree items
	 * @return true if any items would have been returned before filtering
	 */
	bool CreateSubObjectItems(AActor* Actor, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const;
	
	/**
	 * Create all component items for an actor if we are showing components and place them in OutItems. Items not added if filtered out.
	 * If components are returned or would be returned before filters, the Actor will be added at the beginning of the array.
	 */
	TArray<FSceneOutlinerTreeItemPtr> ConditionallyCreateActorAndComponentItems(AActor* Actor) const;

	// Update the hierarchy when actor or world changing events occur
	void OnWorldPartitionCreated(UWorld* InWorld);

	void OnLevelActorAdded(AActor* InActor);
	void OnLevelActorDeleted(AActor* InActor);
		
	void OnLevelActorAttached(AActor* InActor, const AActor* InParent);
	void OnLevelActorDetached(AActor* InActor, const AActor* InParent);

	void OnLevelActorFolderChanged(const AActor* InActor, FName OldPath);

	void OnLoadedActorAdded(AActor& InActor);
	void OnLoadedActorRemoved(AActor& InActor);

	void OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* ActorDescInstance);
	
	void OnComponentsUpdated();

	void OnLevelActorListChanged();
	
	void OnActorFolderAdded(UActorFolder* InActorFolder);
	void OnActorFoldersUpdatedEvent(ULevel* InLevel);
		
	void OnLevelAdded(ULevel* InLevel, UWorld* InWorld);
	void OnLevelRemoved(ULevel* InLevel, UWorld* InWorld);

	void OnWorldPartitionInitialized(UWorldPartition* InWorldPartition);
	void OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition);

	/** Called when a folder is to be created */
	void OnBroadcastFolderCreate(UWorld& InWorld, const FFolder& InNewFolder);

	/** Called when a folder is to be moved */
	void OnBroadcastFolderMove(UWorld& InWorld, const FFolder& InOldFolder, const FFolder& InNewFolder);

	/** Called when a folder is to be deleted */
	void OnBroadcastFolderDelete(UWorld& InWorld, const FFolder& InFolder);

	/** Send a an event indicating a full refresh of the hierarchy is required */
	void FullRefreshEvent();

	TWeakObjectPtr<UWorld> RepresentingWorld;
	
	bool bShowingComponents = true;
	bool bShowingOnlyActorWithValidComponents = true;
	bool bShowingLevelInstances = false;
	bool bShowingUnloadedActors = false;
	bool bShowingEmptyFolders = false;

	FObjectMixerOutlinerHierarchy(ISceneOutlinerMode* Mode, const TWeakObjectPtr<UWorld>& Worlds);
};
