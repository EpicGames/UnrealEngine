// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorMode.h"
#include "Containers/Map.h"
#include "EditorConfigBase.h"
#include "ISceneOutlinerTreeItem.h"
#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "WorldPartition/WorldPartitionHandle.h"

#include "ObjectMixerOutlinerMode.generated.h"

#define UE_API OBJECTMIXEREDITOR_API

class FObjectMixerEditorList;
class FObjectMixerOutlinerMode;
class IWorldPartitionEditorModule;

namespace ObjectMixerOutliner
{
	/** A row selector that depends on whether hybrid rows are allowed */
	struct FHybridRowSelector
	{
		FHybridRowSelector(const FObjectMixerOutlinerMode* Mode);

		bool ShouldAllowHybridRows() const { return bAllowHybridRows; }

	private:
		bool bAllowHybridRows = true;
	};

	struct FWeakActorSelectorAcceptingComponents
	{
		bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, TWeakObjectPtr<AActor>& DataOut) const;
	};

	struct FComponentSelector : FHybridRowSelector
	{
		using FHybridRowSelector::FHybridRowSelector;

		bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, UActorComponent*& DataOut) const;
	};

	/** Functor which can be used to get weak actor pointers from a selection */
	struct FWeakActorSelector 
	{
		bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, TWeakObjectPtr<AActor>& DataOut) const;
	};

	/** Functor which can be used to get actors from a selection including component parents */
	struct FActorSelector : FHybridRowSelector
	{
		using FHybridRowSelector::FHybridRowSelector;

		bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, AActor*& ActorPtrOut) const;
	};

	/** Functor which can be used to get actor descriptors from a selection  */
	struct FActorHandleSelector
	{
		bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, FWorldPartitionHandle& ActorHandleOut) const;
	};

	struct FFolderPathSelector
	{
		bool operator()(TWeakPtr<ISceneOutlinerTreeItem> Item, FFolder& DataOut) const;
	};

	struct FObjectSelector
	{
		bool operator()(TWeakPtr<ISceneOutlinerTreeItem> Item, UObject*& OutObject);
	};
}

USTRUCT()
struct FObjectMixerOutlinerModeConfig
{
	GENERATED_BODY()

	/** True when the Scene Outliner is hiding temporary/run-time Actors */
	UPROPERTY()
	bool bHideTemporaryActors = false;

	/** True when the Scene Outliner is showing only Actors that exist in the current level */
	UPROPERTY()
	bool bShowOnlyActorsInCurrentLevel = false;

	/** True when the Scene Outliner is showing only Actors that exist in the current data layers */
	UPROPERTY()
	bool bShowOnlyActorsInCurrentDataLayers = false;

	/** True when the Scene Outliner is showing only Actors that exist in the current content bundle */
	UPROPERTY()
	bool bShowOnlyActorsInCurrentContentBundle = false;

	/** True when the Scene Outliner is only displaying selected Actors */
	UPROPERTY()
	bool bShowOnlySelectedActors = false;

	/** True when the Scene Outliner is not displaying Actor Components*/
	UPROPERTY()
	bool bHideActorComponents = true;

	/** True when the Scene Outliner is not displaying LevelInstances */
	UPROPERTY()
	bool bHideLevelInstanceHierarchy = false;

	/** True when the Scene Outliner is not displaying unloaded actors */
	UPROPERTY()
	bool bHideUnloadedActors = false;

	/** True when the Scene Outliner is not displaying empty folders */
	UPROPERTY()
	bool bHideEmptyFolders = false;

	/** True when the Scene Outliner updates when an actor is selected in the viewport */
	UPROPERTY()
	bool bAlwaysFrameSelection = true;
};

UCLASS(EditorConfig="ObjectMixerOutlinerMode")
class UObjectMixerOutlinerModeEditorConfig : public UEditorConfigBase
{
	GENERATED_BODY()
	
public:

	static void Initialize()
	{
		if(!Instance)
		{
			Instance = NewObject<UObjectMixerOutlinerModeEditorConfig>(); 
			Instance->AddToRoot();
		}
	}

	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FObjectMixerOutlinerModeConfig> Browsers;

private:

	static TObjectPtr<UObjectMixerOutlinerModeEditorConfig> Instance;
};

struct FObjectMixerOutlinerModeParams : FActorModeParams
{
	FObjectMixerOutlinerModeParams() {}

	FObjectMixerOutlinerModeParams(
		SSceneOutliner* InSceneOutliner,
		const TWeakObjectPtr<UWorld>& InSpecifiedWorldToDisplay = nullptr,
		bool bInHideComponents = true, bool bInHideLevelInstanceHierarchy = true,
		bool bInHideUnloadedActors = true, bool bInHideEmptyFolders = true, bool bInHideActorWithNoComponent = true)
		: FActorModeParams(InSceneOutliner, InSpecifiedWorldToDisplay, bInHideComponents, bInHideLevelInstanceHierarchy, bInHideUnloadedActors, bInHideEmptyFolders)
	{
		bHideActorWithNoComponent = bInHideActorWithNoComponent;
	}
};

class FObjectMixerOutlinerMode : public FActorMode
{
public:
	
	struct FFilterClassSelectionInfo
	{
		UClass* Class;
		bool bIsUserSelected;
	};
	
	UE_API FObjectMixerOutlinerMode(const FObjectMixerOutlinerModeParams& Params, const TSharedRef<FObjectMixerEditorList> InListModel);
	UE_API virtual ~FObjectMixerOutlinerMode();

	TWeakPtr<FObjectMixerEditorList> GetListModelPtr() const
	{
		return ListModelPtr;
	}

	SSceneOutliner* GetSceneOutliner() const
	{
		return SceneOutliner;
	}
	
	/**
	 * Determines the style of the tree (flat list or hierarchy)
	 */
	UE_API EObjectMixerTreeViewMode GetTreeViewMode() const;
	/**
	 * Determine the style of the tree (flat list or hierarchy)
	 */
	UE_API void SetTreeViewMode(EObjectMixerTreeViewMode InViewMode);

	UE_API UWorld* GetRepresentingWorld();

	/* Begin ISceneOutlinerMode Interface */
	virtual bool IsInteractive() const override { return true; }
	UE_API virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() override;
	UE_API virtual void Rebuild() override;
	UE_API virtual int32 GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const override;
	UE_API virtual FCreateSceneOutlinerMode CreateFolderPickerMode(const FFolder::FRootObject& InRootObject = FFolder::GetInvalidRootObject()) const override;
	UE_API virtual void CreateViewContent(FMenuBuilder& MenuBuilder) override;
	UE_API virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const override;
	UE_API virtual bool ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const override;
	UE_API virtual FSceneOutlinerDragValidationInfo ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const override;
	UE_API virtual TSharedPtr<SWidget> CreateContextMenu() override;
	UE_API virtual void OnItemAdded(FSceneOutlinerTreeItemPtr Item) override;
	UE_API virtual void OnItemRemoved(FSceneOutlinerTreeItemPtr Item) override;
	UE_API virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;
	UE_API virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override;
	UE_API virtual void OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType) override;
	UE_API virtual void OnItemPassesFilters(const ISceneOutlinerTreeItem& Item) override;
	UE_API virtual FReply OnKeyDown(const FKeyEvent& InKeyEvent) override;
	UE_API virtual void OnDuplicateSelected() override;
	UE_API virtual void OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const override;
	UE_API virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const override;
	UE_API virtual FText GetStatusText() const override;
	UE_API virtual FSlateColor GetStatusTextColor() const override;
	virtual ESelectionMode::Type GetSelectionMode() const override { return ESelectionMode::Multi; }
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual bool ShouldShowFolders() const override { return GetTreeViewMode() == EObjectMixerTreeViewMode::Folders; }
	virtual bool SupportsCreateNewFolder() const override { return true; }
	virtual bool ShowStatusBar() const override { return true; }
	virtual bool ShowViewButton() const override { return true; }
	virtual bool ShowFilterOptions() const override { return true; }
	UE_API virtual bool CanDelete() const override;
	UE_API virtual bool CanRename() const override;
	UE_API virtual bool CanCut() const override;
	UE_API virtual bool CanCopy() const override;
	UE_API virtual bool CanPaste() const override;
	virtual bool CanSupportDragAndDrop() const override { return true; }
	virtual bool CanCustomizeToolbar() const override { return true; }
	UE_API virtual bool HasErrors() const override;
	UE_API virtual FText GetErrorsText() const override;
	UE_API virtual void RepairErrors() const override;	
	UE_API virtual FFolder CreateNewFolder() override;
	UE_API virtual FFolder GetFolder(const FFolder& ParentPath, const FName& LeafName) override;
	UE_API virtual bool CreateFolder(const FFolder& NewFolder) override;
	UE_API virtual bool ReparentItemToFolder(const FFolder& FolderPath, const FSceneOutlinerTreeItemPtr& Item) override;
	UE_API virtual void SelectFoldersDescendants(const TArray<FFolderTreeItem*>& FolderItems, bool bSelectImmediateChildrenOnly) override;
	UE_API virtual void PinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) override;
	UE_API virtual void UnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) override;
	UE_API virtual bool CanPinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const override;
	UE_API virtual bool CanUnpinItems(const TArray<FSceneOutlinerTreeItemPtr>& InItems) const override;
	UE_API virtual void SynchronizeSelection() override;
	/* End ISceneOutlinerMode Interface */

	/* External events this mode must respond to */

	/** Called by the engine when a component is updated */
	UE_API void OnComponentsUpdated();
	/** Called by the engine when an actor is deleted */
	UE_API void OnLevelActorDeleted(AActor* Actor);

	/** Called by the editor to allow selection of unloaded actors */
	UE_API void OnSelectUnloadedActors(const TArray<FGuid>& ActorGuids);
	
	/** Called when an actor desc instance is removed */
	UE_API void OnActorDescInstanceRemoved(FWorldPartitionActorDescInstance* InActorDescInstance);
	
	UE_DEPRECATED(5.4, "Use OnActorDescInstanceRemoved instead")
	void OnActorDescRemoved(FWorldPartitionActorDesc* InActorDesc) {}

	/** Called by engine when edit cut actors begins */
	UE_API void OnEditCutActorsBegin();

	/** Called by engine when edit cut actors ends */
	UE_API void OnEditCutActorsEnd();

	/** Called by engine when edit copy actors begins */
	UE_API void OnEditCopyActorsBegin();

	/** Called by engine when edit copy actors ends */
	UE_API void OnEditCopyActorsEnd();

	/** Called by engine when edit paste actors begins */
	UE_API void OnEditPasteActorsBegin();

	/** Called by engine when edit paste actors ends */
	UE_API void OnEditPasteActorsEnd();

	/** Called by engine when edit duplicate actors begins */
	UE_API void OnDuplicateActorsBegin();

	/** Called by engine when edit duplicate actors ends */
	UE_API void OnDuplicateActorsEnd();

	/** Called by engine when edit delete actors begins */
	UE_API void OnDeleteActorsBegin();

	/** Called by engine when edit delete actors ends */
	UE_API void OnDeleteActorsEnd();
	
	/** Function called by the Outliner Filter Bar to compare an item with Type Filters*/
	UE_API virtual bool CompareItemWithClassName(SceneOutliner::FilterBarType InItem, const TSet<FTopLevelAssetPath>&) const override;

	/** Check whether hybrid rows are allowed for the associated object list */
	UE_API bool ShouldAllowHybridRows() const;
	
protected:

	/* Events */

	UE_API void OnMapChange(uint32 MapFlags);
	UE_API void OnNewCurrentLevel();

	UE_API void OnEditorSelectionChanged();
	UE_API void OnActorLabelChanged(AActor* ChangedActor);
	UE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);
	UE_API void OnLevelActorRequestsRename(const AActor* Actor);
	UE_API void OnPostLoadMapWithWorld(UWorld* World);

	UE_API bool ShouldSyncSelectionToEditor();
	UE_API bool ShouldSyncSelectionFromEditor();
	UE_API void SynchronizeAllSelectionsToEditor();
	UE_API bool HasActorSelectionChanged(TArray<AActor*>& OutSelectedActors, bool& bOutAreAnyInPIE);
	UE_API bool HasComponentSelectionChanged(TArray<UActorComponent*>& OutSelectedComponents, bool& bOutAreAnyInPIE);
	UE_API void SelectActorsInEditor(const TArray<AActor*>& InSelectedActors, bool bShouldSelect, bool bSelectEvenIfHidden);
	UE_API void SelectComponentsInEditor(const TArray<UActorComponent*>& InSelectedComponents, bool bShouldSelect, bool bSelectEvenIfHidden);
	UE_API void SelectSubObjectsInEditor(const TArray<UObject*>& InSelectedSubObjects, bool bShouldSelect, bool bSelectEvenIfHidden);
	UE_API void SelectActorsInMixer(const TArray<AActor*>& InSelectedActors, bool bShouldSelect, bool bSelectEvenIfHidden);
	UE_API void SelectComponentsInMixer(const TArray<UActorComponent*>& InSelectedComponents, bool bShouldSelect, bool bSelectEvenIfHidden);

	/** Get the list of actors selected in the editor */
	UE_API TArray<AActor*> GetSelectedActorsInEditor() const;

	/** Get the list of components selected in the editor */
	UE_API TArray<UActorComponent*> GetSelectedComponentsInEditor() const;

	/** Gets the list of actor sub-objects that are selected in the editor. */
	UE_API TArray<UObject*> GetSelectedSubObjectsInEditor() const;

	/** Check if an object is a member of one of the classes our list model is filtered to */
	UE_API bool GetIsObjectOfFilteredClass(const UObject* Object) const;
	
	/** Build and up the context menu */
	UE_API TSharedPtr<SWidget> BuildContextMenu();
	/** Register the context menu with the engine */
	static UE_API void RegisterContextMenu();
	UE_API bool CanPasteFoldersOnlyFromClipboard() const;

	UE_API bool GetFolderNamesFromPayload(const FSceneOutlinerDragDropPayload& InPayload, TArray<FName>& OutFolders, FFolder::FRootObject& OutCommonRootObject) const;
	UE_API FFolder GetWorldDefaultRootFolder() const;

	/** Synchronize both the actor and component selection with the editor, with components taking priority over actors */
	UE_API void SynchronizeComponentAndActorSelection();
	UE_API void SynchronizeSelectedActorDescs();

	/** Synchronizes the object mixer's selection with any sub-objects selected elsewhere in the editor */
	UE_API void SynchronizeSubObjectSelection();

	UE_API void OnActorEditorContextSubsystemChanged();

	/** Filter factories */
	static UE_API TSharedRef<FSceneOutlinerFilter> CreateShowOnlySelectedActorsFilter();
	static UE_API TSharedRef<FSceneOutlinerFilter> CreateHideTemporaryActorsFilter();
	static UE_API TSharedRef<FSceneOutlinerFilter> CreateIsInCurrentLevelFilter();
	static UE_API TSharedRef<FSceneOutlinerFilter> CreateIsInCurrentDataLayersFilter();
	static UE_API TSharedRef<FSceneOutlinerFilter> CreateHideComponentsFilter();
	static UE_API TSharedRef<FSceneOutlinerFilter> CreateHideLevelInstancesFilter();
	static UE_API TSharedRef<FSceneOutlinerFilter> CreateHideUnloadedActorsFilter();
	static UE_API TSharedRef<FSceneOutlinerFilter> CreateHideEmptyFoldersFilter();
	UE_API TSharedRef<FSceneOutlinerFilter> CreateIsInCurrentContentBundleFilter();

	/** Functions to expose selection framing to the UI */
	UE_API void OnToggleAlwaysFrameSelection();
	UE_API bool ShouldAlwaysFrameSelection();

	/**
	 * Get a mutable version of the ActorBrowser config for setting values.
	 * @returns		The config for this ActorBrowser.
	 * @note		If OutlinerIdentifier is not set for this outliner, it is not possible to store settings.
	 */
	UE_API FObjectMixerOutlinerModeConfig* GetMutableConfig() const;

	/**
	 * Get a const version of the ActorBrowser config for getting values.
	 * @returns		The config for this ActorBrowser.
	 * @note		If OutlinerIdentifier is not set for this outliner, it is not possible to store settings.
	 */
	UE_API const FObjectMixerOutlinerModeConfig* GetConstConfig() const;

	/** Save the config for this ActorBrowser */
	UE_API void SaveConfig();

	IWorldPartitionEditorModule* WorldPartitionEditorModule;
	/** Number of actors (including unloaded) which have passed through the filters */
	uint32 FilteredActorCount = 0;
	/** Number of unloaded actors which have passed through all the filters */
	uint32 FilteredUnloadedActorCount = 0;
	/** List of unloaded actors which passed through the regular filters and may or may not have passed the search filter */
	TSet<FWorldPartitionHandle> ApplicableUnloadedActors;
	/** List of actors which passed the regular filters and may or may not have passed the search filter */
	TSet<TWeakObjectPtr<AActor>> ApplicableActors;

	bool bRepresentingWorldGameWorld = false;
	bool bRepresentingWorldPartitionedWorld = false;

	TWeakPtr<FObjectMixerEditorList> ListModelPtr;
	
	TArray<FFilterClassSelectionInfo> FilterClassSelectionInfos;
	UE_API TSharedRef<SWidget> OnGenerateFilterClassMenu();

	/** Used in case of a selection sync override. */
	
	bool bShouldTemporarilyForceSelectionSyncFromEditor = false;
	bool bShouldTemporarilyForceSelectionSyncToEditor = false;

	/** The last selected tree item IDs, stored here so we can maintain the selection across refreshes when not synced with the editor. */
	TArray<FSceneOutlinerTreeItemID> SelectedIDs;
};

#undef UE_API
