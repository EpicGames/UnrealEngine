// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutliner.h"
#include "SceneOutlinerFwd.h"
#include "SceneOutlinerGutter.h"
#include "SceneOutlinerTextInfoColumn.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class SWidget;
template<typename ItemType> class STableRow;

class FObjectMixerOutlinerVisibilityColumn : public FSceneOutlinerGutter
{
public:
	FObjectMixerOutlinerVisibilityColumn(ISceneOutliner& SceneOutliner) : FSceneOutlinerGutter(SceneOutliner)
	{
		const FName VisibleHoveredBrushName = TEXT("Level.VisibleHighlightIcon16x");
		const FName VisibleNotHoveredBrushName = TEXT("Level.VisibleIcon16x");
		const FName NotVisibleHoveredBrushName = TEXT("Level.NotVisibleHighlightIcon16x");
		const FName NotVisibleNotHoveredBrushName = TEXT("Level.NotVisibleIcon16x");

		VisibleHoveredBrush = FAppStyle::Get().GetBrush(VisibleHoveredBrushName);
		VisibleNotHoveredBrush = FAppStyle::Get().GetBrush(VisibleNotHoveredBrushName);
		NotVisibleHoveredBrush = FAppStyle::Get().GetBrush(NotVisibleHoveredBrushName);
		NotVisibleNotHoveredBrush = FAppStyle::Get().GetBrush(NotVisibleNotHoveredBrushName);
	}
	virtual ~FObjectMixerOutlinerVisibilityColumn() override = default;

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////
	
protected:
	
	const FSlateBrush* VisibleHoveredBrush = nullptr;
	const FSlateBrush* VisibleNotHoveredBrush = nullptr;
	const FSlateBrush* NotVisibleHoveredBrush = nullptr;
	const FSlateBrush* NotVisibleNotHoveredBrush = nullptr;
};

class FObjectMixerOutlinerSoloColumn : public ISceneOutlinerColumn
{
public:
	FObjectMixerOutlinerSoloColumn(ISceneOutliner& SceneOutliner)
	{
		WeakOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());
	}
	virtual ~FObjectMixerOutlinerSoloColumn() override = default;
	static FName GetID();

	static FText GetLocalizedColumnName();

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation
	virtual FName GetColumnID() override { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return true; }
	virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const override;
	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////

protected:

	FSlateColor GetForegroundColor(
		FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>* Row) const;

	void OnClickSoloIcon(const FSceneOutlinerTreeItemRef RowPtr);
	
	TWeakPtr<ISceneOutliner> WeakOutliner = nullptr;
};

class FObjectMixerOutlinerPropertyColumn : public ISceneOutlinerColumn
{
public:
	FObjectMixerOutlinerPropertyColumn(ISceneOutliner& SceneOutliner, FProperty* InProperty)
	{
		Property = InProperty;
		WeakOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());
	}
	virtual ~FObjectMixerOutlinerPropertyColumn() override = default;
	static FName GetID(const FProperty* InProperty);

	static FText GetDisplayNameText(const FProperty* InProperty);

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation
	virtual FName GetColumnID() override { return GetID(Property); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return false; }
	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////
	
protected:

	void OnPropertyChanged(const FPropertyChangedEvent& Event, TWeakPtr<ISceneOutlinerTreeItem> WeakRowPtr) const;
	
	FProperty* Property = nullptr;
	TWeakPtr<ISceneOutliner> WeakOutliner = nullptr;
};

class FObjectMixerOutlinerTextInfoColumn : public FTextInfoColumn
{
public:
	FObjectMixerOutlinerTextInfoColumn(
		ISceneOutliner& SceneOutliner, const FName InColumnName,
		const FGetTextForItem& InGetTextForItem, const FText InColumnToolTip)
		: FTextInfoColumn(SceneOutliner, InColumnName, InGetTextForItem, InColumnToolTip)
	{}
};
