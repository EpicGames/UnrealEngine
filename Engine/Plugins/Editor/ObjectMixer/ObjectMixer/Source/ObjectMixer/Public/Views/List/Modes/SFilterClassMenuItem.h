// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Find.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "Views/List/Modes/ObjectMixerOutlinerMode.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"

struct FAssetClassMap
{
	TObjectPtr<UClass> Class = nullptr;
	FAssetData AssetData;

	bool operator==(const FAssetClassMap& Other) const
	{
		return Class == Other.Class;
	}
};

class SFilterClassMenuItem : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SFilterClassMenuItem)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FAssetClassMap AssetClassMap, const bool bIsDefaultClass,
		TArray<FObjectMixerOutlinerMode::FFilterClassSelectionInfo>& FilterClassSelectionInfos, const FText TooltipText)
	{
		const bool bHasValidAssetData = AssetClassMap.AssetData.IsValid();

		FText ToolTipText;
		if (bHasValidAssetData)
		{
			const TSharedPtr<FUICommandInfo> FindInContentBrowserCommand = FGlobalEditorCommonCommands::Get().FindInContentBrowser;
			checkf(FindInContentBrowserCommand.IsValid(), TEXT("FGlobalEditorCommonCommands::FindInContentBrowser command not found!"));
			ToolTipText = FindInContentBrowserCommand->GetDescription();
		}
		else
		{
			ToolTipText = NSLOCTEXT("SFilterClassMenuItem", "NoBlueprintFilterFound", "This filter class is not a Blueprint class.");
		}

		ChildSlot
		[
			SNew(SHorizontalBox)
			.ToolTipText(TooltipText)
			.IsEnabled(!bIsDefaultClass)

			+SHorizontalBox::Slot()
			.Padding(FMargin(0, 0, 8, 0))
			.AutoWidth()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged_Lambda([this, AssetClassMap, &FilterClassSelectionInfos](ECheckBoxState NewState)
			   {
					if (FObjectMixerOutlinerMode::FFilterClassSelectionInfo* Match =
						Algo::FindByPredicate(FilterClassSelectionInfos,
							[AssetClassMap](const FObjectMixerOutlinerMode::FFilterClassSelectionInfo& Other)
							{
								return Other.Class == AssetClassMap.Class;
							}))
					{
						   Match->bIsUserSelected = !Match->bIsUserSelected;
					}
			   })
			   .IsChecked_Lambda([this, AssetClassMap, &FilterClassSelectionInfos]()
			   {
		   			bool bShouldBeChecked = false;
		   		
					if (const FObjectMixerOutlinerMode::FFilterClassSelectionInfo* Match =
			   			Algo::FindByPredicate(FilterClassSelectionInfos,
							[AssetClassMap](const FObjectMixerOutlinerMode::FFilterClassSelectionInfo& Other)
							{
								return Other.Class == AssetClassMap.Class;
							}))
				   {
					   bShouldBeChecked = Match->bIsUserSelected;
				   }
				   return bShouldBeChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			   })
			]

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0, 0, 8, 0))
			[
				SNew(STextBlock)
				.Text(FText::FromString(
					AssetClassMap.Class->GetName().EndsWith(TEXT("_C")) ? AssetClassMap.Class->GetName().LeftChop(2) : AssetClassMap.Class->GetName()))
			]

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()	
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ContentPadding(FMargin(2))
				.IsEnabled(bHasValidAssetData)
				.ToolTipText(ToolTipText)
				.OnClicked_Lambda([AssetClassMap]()
				{
					const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
					ContentBrowserModule.Get().SyncBrowserToAssets({AssetClassMap.AssetData});

					return FReply::Handled();
				})
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("SystemWideCommands.FindInContentBrowser.Small"))
				]
			]
		];
	}
};
