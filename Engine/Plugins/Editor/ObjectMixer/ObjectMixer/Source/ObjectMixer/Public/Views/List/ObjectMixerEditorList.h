// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectMixerEditorListRowData.h"
#include "SelectionInterface/IObjectMixerSelectionInterface.h"

#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"

#define UE_API OBJECTMIXEREDITOR_API

class FObjectMixerEditorModule;
class SObjectMixerEditorList;
class UObjectMixerEditorSerializedData;

DECLARE_MULTICAST_DELEGATE(FOnPreFilterChange)
DECLARE_MULTICAST_DELEGATE(FOnPostFilterChange)


class FObjectMixerEditorList : public TSharedFromThis<FObjectMixerEditorList>, public FGCObject
{
public:

	UE_API FObjectMixerEditorList(const FName InModuleName, TSharedPtr<IObjectMixerSelectionInterface> InSelectionInterface = nullptr);

	UE_API virtual ~FObjectMixerEditorList();

	UE_API void Initialize();
	
	UE_API void RegisterAndMapContextMenuCommands();

	/** Creates a new widget only if one does not already exist. Otherwise returns the existing widget. */
	UE_API TSharedRef<SWidget> GetOrCreateWidget();

	/** Creates a new widget, replacing the existing one's pointer. Not useful to call alone, use RequestRegenerateListWidget */
	UE_API TSharedRef<SWidget> CreateWidget();

	/** Calls back to the module to replace the widget in the dock tab. Call when columns or the object filters have changed. */
	UE_API void RequestRegenerateListWidget();

	/** Rebuild the list items and refresh the list. Call when adding or removing items. */
	UE_API void RequestRebuildList() const;

	/**
	 * Refresh list filters and sorting.
	 * Useful for when the list state has gone stale but the variable count has not changed.
	 */
	UE_API void RefreshList() const;

	/** Get the selection interface used to synchronize this with the rest of the editor */
	TSharedPtr<IObjectMixerSelectionInterface> GetSelectionInterface() const { return SelectionInterface; }

	UE_API void BuildPerformanceCache();

	UE_API bool ShouldShowTransientObjects() const;

	/** Called when the Rename command is executed from the UI or hotkey. */
	UE_API void OnRenameCommand();
	
	UE_API void RebuildCollectionSelector();

	UE_API void SetDefaultFilterClass(UClass* InNewClass);
	UE_API bool IsClassSelected(UClass* InClass) const;

	UE_API const TArray<TObjectPtr<UObjectMixerObjectFilter>>& GetObjectFilterInstances();

	UE_API const UObjectMixerObjectFilter* GetMainObjectFilterInstance();

	UE_API void CacheObjectFilterInstances();

	/** Get the style of the tree (flat list or hierarchy) */
	EObjectMixerTreeViewMode GetTreeViewMode()
	{
		return TreeViewMode;
	}
	/** Set the style of the tree (flat list or hierarchy)  */
	void SetTreeViewMode(EObjectMixerTreeViewMode InViewMode)
	{
		TreeViewMode = InViewMode;
		RequestRebuildList();
	}

	/**
	 * Force returns result from Filter->ForceGetObjectClassesToFilter.
	 * Generally, you want to get this from the public performance cache.
	 */
	UE_API TSet<UClass*> ForceGetObjectClassesToFilter();

	/**
	 * Force returns result from Filter->ForceGetObjectClassesToPlace.
	 * Generally, you want to get this from the public performance cache.
	 */
	UE_API TSet<TSubclassOf<AActor>> ForceGetObjectClassesToPlace();

	const TArray<TSubclassOf<UObjectMixerObjectFilter>>& GetObjectFilterClasses() const
	{
		return ObjectFilterClasses;
	}

	UE_API void AddObjectFilterClass(UClass* InObjectFilterClass, const bool bShouldRebuild = true);

	UE_API void RemoveObjectFilterClass(UClass* InObjectFilterClass, const bool bCacheAndRebuild = true);

	void ResetObjectFilterClasses(const bool bCacheAndRebuild = true)
	{
		ObjectFilterClasses.Empty(ObjectFilterClasses.Num());

		if (bCacheAndRebuild)
		{
			CacheAndRebuildFilters();
		}
	}

	void CacheAndRebuildFilters(const bool bShouldRegenerateWidget = false)
	{
		CacheObjectFilterInstances();

		if (bShouldRegenerateWidget)
		{
			RequestRegenerateListWidget();
			return;
		}
		
		RequestRebuildList();
	}

	/** Used as a way to differentiate different subclasses of the Object Mixer module */
	FName GetModuleName() const
	{
		return ModuleName;
	}

	UE_API FObjectMixerEditorModule* GetModulePtr() const;

	// User Collections

	/** Get a pointer to the UObjectMixerEditorSerializedData object along with the name of the filter represented by this ListModel instance. */
	UE_API UObjectMixerEditorSerializedData* GetSerializedData() const;
	UE_API bool RequestAddObjectsToCollection(const FName& CollectionName, const TSet<FSoftObjectPath>& ObjectsToAdd) const;
	UE_API bool RequestRemoveObjectsFromCollection(const FName& CollectionName, const TSet<FSoftObjectPath>& ObjectsToRemove) const;
	UE_API bool RequestRemoveCollection(const FName& CollectionName) const;
	UE_API bool RequestDuplicateCollection(const FName& CollectionToDuplicateName, FName& DesiredDuplicateName) const;
	UE_API bool RequestReorderCollection(const FName& CollectionToMoveName, const FName& CollectionInsertBeforeName) const;
	UE_API bool RequestRenameCollection(const FName& CollectionNameToRename, const FName& NewCollectionName) const;
	UE_API bool DoesCollectionExist(const FName& CollectionName) const;
	UE_API bool IsObjectInCollection(const FName& CollectionName, const FSoftObjectPath& InObject) const;
	UE_API TSet<FName> GetCollectionsForObject(const FSoftObjectPath& InObject) const;
	UE_API TArray<FName> GetAllCollectionNames() const;

	/**
	 * This is the filter class used to initialize the ListModel.
	 * This filter class cannot be turned off by the end user.
	 */
	UE_API const TSubclassOf<UObjectMixerObjectFilter>& GetDefaultFilterClass() const;

	UE_API void OnPostFilterChange();

	FOnPreFilterChange OnPreFilterChangeDelegate;
	FOnPostFilterChange OnPostFilterChangeDelegate;

	TSharedPtr<FUICommandList> ObjectMixerElementEditCommands;
	TSharedPtr<FUICommandList> ObjectMixerFolderEditCommands;

	UE_API void FlushWidget();

	[[nodiscard]] UE_API TArray<TSharedPtr<ISceneOutlinerTreeItem>> GetSelectedTreeViewItems() const;
	UE_API int32 GetSelectedTreeViewItemCount() const;

	UE_API TSet<TSharedPtr<ISceneOutlinerTreeItem>> GetSoloRows() const;
	UE_API void ClearSoloRows();

	/** Returns true if at least one row is set to Solo. */
	UE_API bool IsListInSoloState() const;

	/**
	 * Determines whether rows' objects should be temporarily hidden in editor based on each row's visibility rules,
	 * then sets each object's visibility in editor.
	 */
	UE_API void EvaluateAndSetEditorVisibilityPerRow();
	
	/** Represents the collections the user has selected in the UI. If empty, "All" is considered as selected. */
	[[nodiscard]] UE_API const TSet<FName>& GetSelectedCollections() const;
	[[nodiscard]] UE_API bool IsCollectionSelected(const FName& CollectionName) const;
	UE_API void SetSelectedCollections(const TSet<FName> InSelectedCollections);
	UE_API void SetCollectionSelected(const FName& CollectionName, const bool bNewSelected);

	// Performance cache
	TSet<UClass*> ObjectClassesToFilterCache;
	TSet<FName> ColumnsToShowByDefaultCache;
	TSet<FName> ColumnsToExcludeCache;
	TSet<FName> ForceAddedColumnsCache;
	EObjectMixerInheritanceInclusionOptions PropertyInheritanceInclusionOptionsCache = EObjectMixerInheritanceInclusionOptions::None;
	bool bShouldIncludeUnsupportedPropertiesCache = false;
	bool bShouldShowTransientObjectsCache = false;

protected:

	/** Bind delegates to refresh the list when editor state changes. */
	UE_API virtual void BindRefreshDelegates();

	/** Unbind delegates to refresh the list when editor state changes. */
	UE_API virtual void UnbindRefreshDelegates();

	/** If a property is changed that has a name found in this set, the list will be refreshed. */
	UE_API TSet<FName> GetPropertiesThatRequireRefresh();
	
	virtual void AddReferencedObjects( FReferenceCollector& Collector )  override
	{
		Collector.AddReferencedObjects(ObjectFilterInstances);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FObjectMixerEditorList");
	}

	/** Represents the collections the user has selected in the UI. If empty, "All" is considered as selected. */
	TSet<FName> SelectedCollections;

	TSharedPtr<SObjectMixerEditorList> ListWidget;
	
	TArray<TObjectPtr<UObjectMixerObjectFilter>> ObjectFilterInstances;

	/**
	 * The classes used to generate property edit columns.
	 * Using an array rather than a set because the first class is considered the 'Main' class which determines some filter behaviours.
	 */
	TArray<TSubclassOf<UObjectMixerObjectFilter>> ObjectFilterClasses;

	/**
	 * If set, this is the filter class used to initialize the ListModel.
	 * This filter class cannot be turned off by the end user.
	 */
	TSubclassOf<UObjectMixerObjectFilter> DefaultFilterClass;

	/**
	 * Determines the style of the tree (flat list or hierarchy)
	 */
	EObjectMixerTreeViewMode TreeViewMode = EObjectMixerTreeViewMode::Folders;

	FName ModuleName = NAME_None;
	
	FDelegateHandle OnBlueprintFilterCompiledHandle;

	/** Interface used to synchronize selection with another part of the editor */
	TSharedPtr<IObjectMixerSelectionInterface> SelectionInterface;

	/** Handles for delegates registered to refresh the list */
	TSet<FDelegateHandle> DelegateHandles;
};

#undef UE_API
