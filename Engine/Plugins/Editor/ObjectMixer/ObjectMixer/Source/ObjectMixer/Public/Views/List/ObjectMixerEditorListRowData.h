// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"

#include "ISceneOutlinerTreeItem.h"
#include "PropertyHandle.h"
#include "SSceneOutliner.h"

#define UE_API OBJECTMIXEREDITOR_API

class SObjectMixerEditorList;

/** Defines data carried by each row type. */
struct FObjectMixerEditorListRowData
{
	struct FTransientEditorVisibilityRules
	{
		/**
		 * If true, the user wants the row to be hidden temporarily in the editor.
		 * This is transient visibility like the eye icon in the Scene Outliner, not the bVisible or bHiddenInGame properties.
		 */
		bool bShouldBeHiddenInEditor = false;

		/**
		 * If true, the user wants the row to have solo visibility. Multiple rows at once can be set to solo.
		 * Solo rows' objects are exclusively visible,
		 * so all other objects found in the panel will be invisible while at least one row is in a solo state.
		 */
		bool bShouldBeSolo = false;
	};

	struct FPropertyPropagationInfo
	{
		FPropertyPropagationInfo() = default;
		
		FSceneOutlinerTreeItemID RowIdentifier;
		FName PropertyName = NAME_None;
		EPropertyValueSetFlags::Type PropertyValueSetFlags = 0;

		bool operator==(const FPropertyPropagationInfo& Other) const
		{
			return PropertyName == Other.PropertyName;
		}

		friend uint32 GetTypeHash (const FPropertyPropagationInfo& Other)
		{
			return GetTypeHash(Other.PropertyName);
		}
	};
	
	FObjectMixerEditorListRowData(
		SSceneOutliner* InSceneOutliner, const FText& InDisplayNameOverride = FText::GetEmpty())
	: DisplayNameOverride(InDisplayNameOverride)
	{
		if (InSceneOutliner)
		{
			SceneOutlinerPtr = StaticCastSharedRef<SSceneOutliner>(InSceneOutliner->AsShared());
		}
	}
	
	FObjectMixerEditorListRowData() = default;

	~FObjectMixerEditorListRowData() = default;

	bool IsValid() const
	{
		return SceneOutlinerPtr.IsValid();
	}

	UE_API const TArray<TObjectPtr<UObjectMixerObjectFilter>>& GetObjectFilterInstances() const;

	UE_API const UObjectMixerObjectFilter* GetMainObjectFilterInstance() const;

	[[nodiscard]] UE_API bool GetIsTreeViewItemExpanded(const TSharedRef<ISceneOutlinerTreeItem> InRow);
	UE_API void SetIsTreeViewItemExpanded(const TSharedRef<ISceneOutlinerTreeItem> InRow, const bool bNewExpanded);

	[[nodiscard]] UE_API bool GetIsSelected(const TSharedRef<ISceneOutlinerTreeItem> InRow);
	UE_API void SetIsSelected(const TSharedRef<ISceneOutlinerTreeItem> InRow, const bool bNewSelected);

	[[nodiscard]] UE_API bool HasAtLeastOneChildThatIsNotSolo(
		const TSharedRef<ISceneOutlinerTreeItem> InRow, const bool bRecursive = true) const;

	[[nodiscard]] const FText& GetDisplayNameOverride() const
	{
		return DisplayNameOverride;
	}

	void SetDisplayNameOverride(const FText& InDisplayNameOverride)
	{
		DisplayNameOverride = InDisplayNameOverride;
	}

	[[nodiscard]] UE_API TWeakPtr<SObjectMixerEditorList> GetListView() const;

	[[nodiscard]] UE_API TArray<TSharedPtr<ISceneOutlinerTreeItem>> GetSelectedTreeViewItems() const;
	
	UE_API void OnChangeVisibility(const FSceneOutlinerTreeItemRef TreeItem, const bool bNewVisible);
	
	[[nodiscard]] UE_API const FTransientEditorVisibilityRules& GetVisibilityRules() const;
	UE_API void SetVisibilityRules(const FTransientEditorVisibilityRules& InVisibilityRules);

	UE_API bool IsUserSetHiddenInEditor() const;
	UE_API void SetUserHiddenInEditor(const bool bNewHidden);
	
	UE_API bool GetRowSoloState() const;
	UE_API void SetRowSoloState(const bool bNewSolo);

	UE_API void ClearSoloRows() const;

	bool GetIsHybridRow() const
	{
		return HybridComponentPath.IsValid();
	}

	UActorComponent* GetHybridComponent() const
	{
		return HybridComponentPath.Get();
	}
	
	/** If this row represents an actor or other container and should show the data for a single child component, define it here. */
	void SetHybridComponent(UActorComponent* InHybridComponent)
	{
		HybridComponentPath = InHybridComponent;
	}

	/** Attempt to propagate a property's new value to other selected rows.
	 * @return Returns true if the property's handle was found or there is no work to do (i.e., no property name specified or this row is not selected)
	 */
	UE_API bool PropagateChangesToSimilarSelectedRowProperties(
		const TSharedRef<ISceneOutlinerTreeItem> InRow, const FPropertyPropagationInfo PropertyPropagationInfo);
	
	TMap<FName, TWeakPtr<IPropertyHandle>> PropertyNamesToHandles;

	TWeakPtr<SSceneOutliner> SceneOutlinerPtr;

protected:
	FTransientEditorVisibilityRules VisibilityRules;

	FText DisplayNameOverride;

	/**
	 * A breadcrumb trail to a single component that matches the selected Object Mixer Filters.
	 * For example, if your filter is looking for lights and you have an actor with a single light component,
	 * this will show the actor and component on the same line rather than requiring the user to unfold the actor row to see one component.
	 */
	TSoftObjectPtr<UActorComponent> HybridComponentPath = nullptr;
};

#undef UE_API
