// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Folder.h"
#include "GameFramework/Actor.h"
#include "ISceneOutlinerTreeItem.h"
#include "Layout/Visibility.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"

struct FObjectMixerEditorListRowFolder;
struct FObjectMixerEditorListRowActor;
struct FObjectMixerEditorListRowComponent;
struct FObjectMixerEditorListRowUObject;
struct FObjectMixerEditorListRowData;
class FObjectMixerEditorList;

namespace FObjectMixerUtils
{
	FObjectMixerEditorListRowFolder* AsFolderRow(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem);
	FObjectMixerEditorListRowActor* AsActorRow(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem);
	FObjectMixerEditorListRowComponent* AsComponentRow(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem);
	FObjectMixerEditorListRowUObject* AsObjectRow(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem);
	
	FObjectMixerEditorListRowData* GetRowData(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem);

	UObject* GetRowObject(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem, const bool bGetHybridRowComponent = false);
	
	[[nodiscard]] AActor* GetSelfOrOuterAsActor(TSharedPtr<ISceneOutlinerTreeItem> InTreeItem);

	bool IsObjectRefInCollection(const FName& CollectionName, const UObject* Object, const TSharedPtr<FObjectMixerEditorList> ListModel);
	bool IsObjectRefInCollection(const FName& CollectionName, TSharedPtr<ISceneOutlinerTreeItem> InTreeItem);

	void SetChildRowsSelected(
		TSharedPtr<ISceneOutlinerTreeItem> InTreeItem, const bool bNewSelected, const bool bRecursive);
}
