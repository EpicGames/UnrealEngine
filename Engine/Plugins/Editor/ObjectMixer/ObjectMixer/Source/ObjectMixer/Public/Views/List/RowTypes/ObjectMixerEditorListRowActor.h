// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/List/ObjectMixerEditorListRowData.h"

#include "ActorTreeItem.h"
#include "GameFramework/Actor.h"
#include "SSceneOutliner.h"

#define UE_API OBJECTMIXEREDITOR_API

struct FObjectMixerEditorListRowActor : FActorTreeItem
{
	explicit FObjectMixerEditorListRowActor(
		AActor* InObject, 
		SSceneOutliner* InSceneOutliner,
		const FText& InDisplayNameOverride = FText::GetEmpty(),
		AActor* InOverrideParent = nullptr)
	: FActorTreeItem(InObject)
	, OriginalObjectSoftPtr(InObject)
	, OverrideParent(InOverrideParent)
	{
		TreeType = Type;
		RowData = FObjectMixerEditorListRowData(InSceneOutliner, InDisplayNameOverride);
	}
	
	FObjectMixerEditorListRowData RowData;
	
	/** Used in scenarios where the original object may be reconstructed or trashed, such as when running a construction script. */
	TSoftObjectPtr<AActor> OriginalObjectSoftPtr;

	/** If set, the row for this actor will be treated as the current row's parent in the hierarchy */
	TSoftObjectPtr<AActor> OverrideParent;

	/* Begin ISceneOutlinerTreeItem Implementation */
	static UE_API const FSceneOutlinerTreeItemType Type;
	UE_API virtual void OnVisibilityChanged(const bool bNewVisibility) override;
	UE_API virtual FSceneOutlinerTreeItemID GetID() const override;
	/* End ISceneOutlinerTreeItem Implementation */
};

#undef UE_API
