// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/List/ObjectMixerEditorListRowData.h"

#include "ComponentTreeItem.h"
#include "SSceneOutliner.h"

#define UE_API OBJECTMIXEREDITOR_API

struct FObjectMixerEditorListRowComponent : FComponentTreeItem
{
	explicit FObjectMixerEditorListRowComponent(
		UActorComponent* InObject, 
		SSceneOutliner* InSceneOutliner, const FText& InDisplayNameOverride = FText::GetEmpty())
	: FComponentTreeItem(InObject)
	, OriginalObjectSoftPtr(InObject)
	{
		TreeType = Type;
		RowData = FObjectMixerEditorListRowData(InSceneOutliner, InDisplayNameOverride);
	}
	
	FObjectMixerEditorListRowData RowData;
	
	/** Used in scenarios where the original object may be reconstructed or trashed, such as when running a construction script. */
	TSoftObjectPtr<UActorComponent> OriginalObjectSoftPtr;
	
	/* Begin ISceneOutlinerTreeItem Implementation */
	static UE_API const FSceneOutlinerTreeItemType Type;
	/* End ISceneOutlinerTreeItem Implementation */
};

#undef UE_API
