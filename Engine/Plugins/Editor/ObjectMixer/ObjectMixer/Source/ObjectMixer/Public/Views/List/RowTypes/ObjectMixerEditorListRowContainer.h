// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"

#define UE_API OBJECTMIXEREDITOR_API

/** Object Mixer row that represents container properties such as arrays that contain subobjects */
struct FObjectMixerEditorListRowContainer : ISceneOutlinerTreeItem
{
public:
	explicit FObjectMixerEditorListRowContainer(UObject* InPropertyOwner, FName InPropertyName, SSceneOutliner* InSceneOutliner);
	
	/* Begin ISceneOutlinerTreeItem Implementation */
	static UE_API const FSceneOutlinerTreeItemType Type;
	UE_API virtual FString GetDisplayString() const override;
	virtual bool CanInteract() const override { return true; }
	virtual bool IsValid() const override { return !PropertyOwner.IsNull() && Property != nullptr; }
	virtual FSceneOutlinerTreeItemID GetID() const override { return IsValid() ? UniqueId : FSceneOutlinerTreeItemID(); }
	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	/* End ISceneOutlinerTreeItem Implementation */

	/** Gets the object that owns the container property being represented by this row */
	UObject* GetPropertyOwner() const { return PropertyOwner.Get(); }

	/** Gets the FProperty being represented by this row */
	const FProperty* GetContainerProperty() const { return Property; }

public:
	/** Optional icon to display in the list row; if not provided, a default icon will be used */
	TAttribute<const FSlateBrush*> PropertyIconOverride;
	
private:
	/** The explicit object whose container property is being represented by this list row */
	TSoftObjectPtr<UObject> PropertyOwner;
	
	/** The FProperty that is being represented by this list row */
	FProperty* Property = nullptr;

	/** The unique ID of this list row in the scene outliner */
	uint64 UniqueId = 0;
};

#undef UE_API