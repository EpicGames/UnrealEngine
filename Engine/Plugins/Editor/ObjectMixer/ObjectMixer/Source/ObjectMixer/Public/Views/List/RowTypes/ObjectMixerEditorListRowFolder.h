// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/List/ObjectMixerEditorListRowData.h"

#include "ActorFolderTreeItem.h"

#include "Folder.h"
#include "SSceneOutliner.h"

#define UE_API OBJECTMIXEREDITOR_API

struct FObjectMixerEditorListRowFolder : FActorFolderTreeItem
{
	explicit FObjectMixerEditorListRowFolder(
		const FFolder InFolder, 
		SSceneOutliner* InSceneOutliner, UWorld* World, const FText& InDisplayNameOverride = FText::GetEmpty())
	: FActorFolderTreeItem(InFolder, World)
	{
		TreeType = Type;
		RowData = FObjectMixerEditorListRowData(InSceneOutliner, InDisplayNameOverride);
	}
	
	FObjectMixerEditorListRowData RowData;

	/* Begin ISceneOutlinerTreeItem Implementation */
	static UE_API const FSceneOutlinerTreeItemType Type;
	UE_API virtual void OnVisibilityChanged(const bool bNewVisibility) override;
	/* End ISceneOutlinerTreeItem Implementation */
};

#undef UE_API
