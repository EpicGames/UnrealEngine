// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/List/ObjectMixerEditorListRowData.h"

#include "Containers/UnrealString.h"
#include "ISceneOutlinerTreeItem.h"
#include "SSceneOutliner.h"
#include "UObject/Object.h"

#define UE_API OBJECTMIXEREDITOR_API

struct FObjectMixerEditorListRowUObject : ISceneOutlinerTreeItem
{	
	explicit FObjectMixerEditorListRowUObject(
		UObject* InObject, 
		SSceneOutliner* InSceneOutliner, const FText& InDisplayNameOverride = FText::GetEmpty())
	: ISceneOutlinerTreeItem(Type)
	, ObjectSoftPtr(InObject)
	, ID(InObject)
	{
		TreeType = Type;
		RowData = FObjectMixerEditorListRowData(InSceneOutliner, InDisplayNameOverride);
	}
	
	FObjectMixerEditorListRowData RowData;

	/** Used in scenarios where the original object may be reconstructed or trashed, such as when running a construction script. */
	TSoftObjectPtr<UObject> ObjectSoftPtr;
	
	/** Constant identifier for this tree item */
	const FObjectKey ID;

	/** If the UObject list row should be displayed as part of a container, this is the container's property name in this object's outer */
	FName ContainerProperty = NAME_None;

	/** If true, the mixer displays rows for all of this object's parents up to its owning actor; otherwise, it is parented directly to its owning actor */
	bool bIncludeObjectStack = false;
	
	/* Begin ISceneOutlinerTreeItem Implementation */
	static UE_API const FSceneOutlinerTreeItemType Type;
	UE_API virtual FString GetDisplayString() const override;
	virtual bool CanInteract() const override { return true; }
	virtual bool IsValid() const override { return !ObjectSoftPtr.IsNull(); }
	virtual FSceneOutlinerTreeItemID GetID() const override { return ID; }
	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	/* End ISceneOutlinerTreeItem Implementation */
};

#undef UE_API
