// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectMixerEditorList.h"
#include "ObjectMixerEditorListRowData.h"
#include "ObjectMixerEditorSerializedData.h"

#include "SSceneOutliner.h"

#include "SObjectMixerEditorList.generated.h"

#define UE_API OBJECTMIXEREDITOR_API

class FObjectMixerEditorList;
class FObjectMixerOutlinerMode;
class SBox;
class SComboButton;
class SSearchBox;
class SHeaderRow;
class SWrapBox;

UENUM()
enum class EListViewColumnType
{
	BuiltIn,
	PropertyGenerated
};

USTRUCT()
struct FObjectMixerSceneOutlinerColumnInfo
{
	GENERATED_BODY()

	/** The pointer to the actual FProperty */
	FProperty* PropertyRef = nullptr;

	/** The FName of the property */
	UPROPERTY()
	FName PropertyName = NAME_None;

	/** The column identifier for the property. Often this is teh same as PropertyName. */
	UPROPERTY()
	FName ColumnID = NAME_None;

	/** What will be displayed in the column header unless another widget is defined. */
	UPROPERTY()
	FText PropertyDisplayText = FText::GetEmpty();

	/** Is this a built-in column, a column displaying a property widget or something else? */
	UPROPERTY()
	EListViewColumnType PropertyType = EListViewColumnType::PropertyGenerated;

	/**
	 * The category that holds the property.
	 * Used for sorting properties in the context menu, similar to the details view.
	 */
	UPROPERTY()
	FName PropertyCategoryName = NAME_None;

	/** If true, this column can be enabled and disabled by the user. */
	UPROPERTY()
	bool bCanBeHidden = true;

	/** If true, this column will be shown in a clean environment or when default properties are restored. */
	UPROPERTY()
	bool bIsDesiredToBeShownByDefault = false;
};

struct FObjectMixerEditorListRowData;

class SObjectMixerEditorList : public SSceneOutliner
{

public:
	DECLARE_EVENT(SObjectMixerEditorList, FSelectionSynchronizedEvent);
	
	// Columns
	static UE_API const FName ItemNameColumnName;
	static UE_API const FName EditorVisibilityColumnName;
	static UE_API const FName EditorVisibilitySoloColumnName;

	UE_API void Construct(const FArguments& InArgs, TSharedRef<FObjectMixerEditorList> ListModel);

	UE_API virtual ~SObjectMixerEditorList() override;

	UE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/**
	 * Determines the style of the tree (flat list or hierarchy)
	 */
	UE_API EObjectMixerTreeViewMode GetTreeViewMode();
	/**
	 * Determine the style of the tree (flat list or hierarchy)
	 */
	UE_API void SetTreeViewMode(EObjectMixerTreeViewMode InViewMode);

	UE_API const TSet<FName>& GetSelectedCollections();
	[[nodiscard]] UE_API bool IsCollectionSelected(const FName& CollectionName);
	
	UE_API void RebuildCollectionSelector();

	UE_API bool RequestRemoveCollection(const FName& CollectionName);
	UE_API bool RequestDuplicateCollection(const FName& CollectionToDuplicateName, FName& DesiredDuplicateName) const;
	UE_API bool RequestRenameCollection(const FName& CollectionNameToRename, const FName& NewCollectionName);
	UE_API bool DoesCollectionExist(const FName& CollectionName) const;
	
	UE_API void OnCollectionCheckedStateChanged(bool bShouldBeChecked, FName CollectionName);
	UE_API ECheckBoxState GetCollectionCheckedState(FName CollectionName) const;

	UE_API FObjectMixerOutlinerMode* GetCastedMode() const;
	UE_API UWorld* GetWorld() const;

	TWeakPtr<FObjectMixerEditorList> GetListModelPtr() const
	{
		return ListModelPtr;
	}

	/**
	 * Refresh filters and sorting.
	 * Useful for when the list state has gone stale but the variable count has not changed.
	 */
	UE_API void RefreshList();

	/**
	 * Regenerate the list items and refresh the list. Call when adding or removing items.
	 */
	UE_API void RequestRebuildList(const FString& InItemToScrollTo = "");

	/** Called when the Rename command is executed from the UI or hotkey. */
	UE_API void OnRenameCommand();

	UE_API void AddToPendingPropertyPropagations(const FObjectMixerEditorListRowData::FPropertyPropagationInfo& InPropagationInfo);

	[[nodiscard]] UE_API TArray<TSharedPtr<ISceneOutlinerTreeItem>> GetSelectedTreeViewItems() const;
	UE_API int32 GetSelectedTreeViewItemCount() const;

	UE_API void SetTreeViewItemSelected(TSharedRef<ISceneOutlinerTreeItem> Item, const bool bNewSelected);

	UE_API bool IsTreeViewItemSelected(TSharedPtr<ISceneOutlinerTreeItem> Item);

	UE_API TSet<TSharedPtr<ISceneOutlinerTreeItem>> GetTreeRootItems() const;
	UE_API TSet<TWeakPtr<ISceneOutlinerTreeItem>> GetWeakTreeRootItems() const;
	
	UE_API TSet<TSharedPtr<ISceneOutlinerTreeItem>> GetSoloRows() const;
	UE_API void ClearSoloRows();

	/** Returns true if at least one row is set to Solo. */
	UE_API bool IsListInSoloState() const;

	/**
	 * Determines whether rows' objects should be temporarily hidden in editor based on each row's visibility rules,
	 * then sets each object's visibility in editor.
	 */
	UE_API void EvaluateAndSetEditorVisibilityPerRow();

	UE_API bool IsTreeViewItemExpanded(const TSharedPtr<ISceneOutlinerTreeItem>& Row) const;
	UE_API void SetTreeViewItemExpanded(const TSharedPtr<ISceneOutlinerTreeItem>& RowToExpand, const bool bNewExpansion) const;

	UE_API void PropagatePropertyChangesToSelectedRows();

	// Columns

	UE_API FObjectMixerSceneOutlinerColumnInfo* GetColumnInfoByPropertyName(const FName& InPropertyName);
	UE_API void RestoreDefaultPropertyColumns();
	UE_API TSharedRef<SWidget> GenerateHeaderRowContextMenu();

	/** Get the event that broadcasts whenever the selection list is synchronized from the editor */
	FSelectionSynchronizedEvent& GetOnSelectionSynchronized() { return OnSelectionSynchronizedEvent; }

	/* Begin SSceneOutliner Interface */
	UE_API virtual void CustomAddToToolbar(TSharedPtr<class SHorizontalBox> Toolbar) override;
	/* End SSceneOutliner Interface */

protected:

	/** A reference to the struct that controls this widget */
	TWeakPtr<FObjectMixerEditorList> ListModelPtr;

	// User Collections
	
	TSharedPtr<SWrapBox> CollectionSelectorBox;

	bool bIsRebuildRequested = false;
	
	TSet<FObjectMixerEditorListRowData::FPropertyPropagationInfo> PendingPropertyPropagations;

	TArray<FObjectMixerSceneOutlinerColumnInfo> HeaderColumnInfos;
	TSharedPtr<SWidget> HeaderRowContextMenuWidget;

	TWeakPtr<SHorizontalBox> ToolbarPtr;

	FSelectionSynchronizedEvent OnSelectionSynchronizedEvent;

	UE_API bool CanCreateFolder() const;
	UE_API TSharedRef<SWidget> OnGenerateAddObjectButtonMenu() const;
	
	/** Disable all collection filters except CollectionToEnableName */
	UE_API void SetSingleCollectionSelection(const FName& CollectionToEnableName = UObjectMixerEditorSerializedData::AllCollectionName);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void OnActorSpawnedOrDestroyed(AActor* Object)
	{
		RequestRebuildList();
	}
	
	/**
	 * Regenerate the list items and refresh the list. Call when adding or removing items.
	 */
	UE_API void RebuildList();

	UE_API void InsertCollectionSelector();

	/**
	 * Only adds properties that pass a series of tests, including having only one unique entry in the column list array.
	 * @param bForceIncludeProperty If true, only Skiplist and Uniqueness tests will be checked, bypassing class, blueprint editability and other requirements.
	 * @param PropertySkipList These property names will be skipped when they are encountered in the iteration.
	 */
	UE_API bool AddUniquePropertyColumnInfo(
		FProperty* Property,
		const bool bForceIncludeProperty = false,
		const TSet<FName>& PropertySkipList = {}
	);

	UE_API FText GetLevelColumnName() const;
	UE_API void CreateActorTextInfoColumns(FSceneOutlinerInitializationOptions& OutInitOptions);
	UE_API void SetupColumns(FSceneOutlinerInitializationOptions& OutInitOptions);
	UE_API void SortColumnsForHeaderContextMenu();
};

#undef UE_API
