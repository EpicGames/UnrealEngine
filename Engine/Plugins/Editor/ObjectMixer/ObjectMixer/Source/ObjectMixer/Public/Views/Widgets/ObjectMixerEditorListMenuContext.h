// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Views/List/ObjectMixerEditorListRowData.h"

#include "ToolMenuEntry.h"
#include "Types/SlateEnums.h"

#include "ObjectMixerEditorListMenuContext.generated.h"	

#define UE_API OBJECTMIXEREDITOR_API

class UToolMenu;

UCLASS(MinimalAPI)
class UObjectMixerEditorListMenuContext : public UObject
{

	GENERATED_BODY()
	
public:

	struct FObjectMixerEditorListMenuContextData
	{
		TArray<TSharedPtr<ISceneOutlinerTreeItem>> SelectedItems;
		TWeakPtr<class FObjectMixerEditorList> ListModelPtr;
	};

	static UE_API void RegisterFoldersOnlyContextMenu();
	static UE_API void RegisterObjectMixerDynamicCollectionsContextMenuExtension(const FName& MenuName);

	FObjectMixerEditorListMenuContextData Data;

private:

	static UE_API bool DoesSelectionHaveType(const FObjectMixerEditorListMenuContextData& InData, UClass* Type);
	static UE_API void AddCollectionWidget(const FName& Key, const FObjectMixerEditorListMenuContextData& ContextData, UToolMenu* Menu);
	static UE_API void CreateSelectCollectionsSubMenu(UToolMenu* Menu, FObjectMixerEditorListMenuContextData ContextData);
	static UE_API void AddCollectionsMenuItem(UToolMenu* InMenu, const FObjectMixerEditorListMenuContextData& ContextData);
	static UE_API void OnTextCommitted(const FText& InText, ETextCommit::Type InCommitType, const FObjectMixerEditorListMenuContextData ContextData);
	
	static UE_API void OnCollectionMenuEntryCheckStateChanged(const FName Key, const FObjectMixerEditorListMenuContextData ContextData);
	static UE_API void AddObjectsToCollection(const FName Key, const FObjectMixerEditorListMenuContextData ContextData);
	static UE_API void RemoveObjectsFromCollection(const FName Key, const FObjectMixerEditorListMenuContextData ContextData);
	static UE_API bool AreAllObjectsInCollection(const FName Key, const FObjectMixerEditorListMenuContextData ContextData);
	static UE_API ECheckBoxState GetCheckStateForCollection(const FName Key, const FObjectMixerEditorListMenuContextData ContextData);

	static UE_API FToolMenuEntry MakeCustomEditMenu(const FObjectMixerEditorListMenuContextData& ContextData);
	static UE_API void ReplaceEditSubMenu(const FObjectMixerEditorListMenuContextData& ContextData);
};

#undef UE_API
