// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectFilter/ObjectMixerEditorObjectFilter.h"

#include "Components/Widget.h"

#include "ObjectMixerEditorUWidget.generated.h"

#define UE_API OBJECTMIXEREDITOR_API

// User Configurable Variables
USTRUCT(BlueprintType)
struct FObjectMixerWidgetUserConfig
{
	GENERATED_BODY()

	/**
	 * Sets the default filter class to determine what objects and properties to display in this Object Mixer instance.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Object Mixer")
	TSubclassOf<UObjectMixerObjectFilter> DefaultFilterClass;
};

/**
 * A UMG widget wrapper for the Object Mixer widget. Uses the same data as the Generic Object Mixer Instance.
 * Only useful in the editor. It is not compatible at runtime.
 */
UCLASS(MinimalAPI, meta = (DisplayName="Object Mixer (Generic)"))
class UObjectMixerEditorUWidget : public UWidget
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

	// UWidget
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Object Mixer")
	FObjectMixerWidgetUserConfig ObjectMixerWidgetUserConfig;
};

#undef UE_API
