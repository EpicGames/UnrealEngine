// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/SCompoundWidget.h"

#define LOCTEXT_NAMESPACE "ObjectMixerEditor"

class SObjectMixerEditorList;
class SInlineEditableTextBlock;
class SMenuAnchor;
class SObjectMixerEditorList;

class SCollectionSelectionButton final : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SCollectionSelectionButton)
	{}

	SLATE_END_ARGS()
	
	void Construct(
		const FArguments& InArgs,
		const TSharedRef<SObjectMixerEditorList> ListWidget,
		const FName InCollectionName);

	TSharedRef<SWidget> GetContextMenu() const;

	void OnEditableTextCommitted(const FText& Text, ETextCommit::Type Type);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	const FSlateBrush* GetBorderBrush() const;
	FSlateColor GetBorderForeground() const;

	/** Whether the associated collection is selected */
	bool GetIsSelected() const;

	TWeakPtr<SObjectMixerEditorList> ListWidgetPtr;
	FName CollectionName;

private:

	bool bIsPressed = false;
	bool bDropIsValid = false;

	FSlateRoundedBoxBrush UncheckedImage = FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.0f);
	FSlateRoundedBoxBrush UncheckedHoveredImage = FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, 1.0f);
	FSlateRoundedBoxBrush UncheckedPressedImage = FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, 1.0f);
	FSlateRoundedBoxBrush UncheckedValidDropImage = FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f, FStyleColors::Input, 1.0f);
	FSlateRoundedBoxBrush CheckedImage = FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f, FStyleColors::Input, 1.0f);
	FSlateRoundedBoxBrush CheckedHoveredImage = FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, 1.0f);
	FSlateRoundedBoxBrush CheckedPressedImage = FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, 1.0f);
	FSlateRoundedBoxBrush CheckedValidDropImage = FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f, FStyleColors::Input, 1.0f);
	
	TSharedPtr<SInlineEditableTextBlock> EditableTextBlock;
	TSharedPtr<SMenuAnchor> MenuAnchor;
};

#undef LOCTEXT_NAMESPACE
