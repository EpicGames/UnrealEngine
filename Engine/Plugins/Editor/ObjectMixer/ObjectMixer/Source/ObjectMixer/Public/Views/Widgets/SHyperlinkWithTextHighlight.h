// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleWidgets.h"
#endif

/**
 * A hyperlink widget is what you would expect from a browser hyperlink.
 * When a hyperlink is clicked in invokes an OnNavigate() delegate.
 */
class SHyperlinkWithTextHighlight
	: public SButton
{
public:

	SLATE_BEGIN_ARGS(SHyperlinkWithTextHighlight)
		: _Text()
		, _Style(&FAppStyle::Get().GetWidgetStyle< FHyperlinkStyle >("Hyperlink"))
		, _UnderlineStyle(nullptr)
		, _Padding()
		, _OnNavigate()
		, _TextShapingMethod()
		, _TextFlowDirection()
		, _HighlightText()
		{}

		SLATE_ATTRIBUTE( FText, Text )
		SLATE_STYLE_ARGUMENT( FHyperlinkStyle, Style )
		SLATE_STYLE_ARGUMENT( FButtonStyle, UnderlineStyle )
		SLATE_ATTRIBUTE( FMargin, Padding )
		SLATE_EVENT( FSimpleDelegate, OnNavigate )
		SLATE_ARGUMENT( TOptional<ETextShapingMethod>, TextShapingMethod )
		SLATE_ARGUMENT( TOptional<ETextFlowDirection>, TextFlowDirection )

		/** Callback to check if the widget is selected, should only be hooked up if parent widget is handling selection or focus. */
		SLATE_EVENT( FIsSelected, IsSelected )

		/** Callback when the text is committed. */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

		/** Highlight this text in the text block */
		SLATE_ATTRIBUTE( FText, HighlightText )
		SLATE_END_ARGS()

	/**
	 * Construct the hyperlink widgets from a declaration
	 * 
	 * @param InArgs    Widget declaration from which to construct the hyperlink.
	 */
	void Construct( const FArguments& InArgs )
	{
		this->OnNavigate = InArgs._OnNavigate;

		check (InArgs._Style);
		const FButtonStyle* UnderlineStyle = InArgs._UnderlineStyle != nullptr ? InArgs._UnderlineStyle : &InArgs._Style->UnderlineStyle;
		TAttribute<FMargin> Padding = InArgs._Padding.IsSet() ? InArgs._Padding : InArgs._Style->Padding;
		
		EditableTextBlockStyle = FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle");
		EditableTextBlockStyle.TextStyle = InArgs._Style->TextStyle;

		SButton::Construct(
			SButton::FArguments()
			.ContentPadding(Padding)
			.ButtonStyle(UnderlineStyle)
			.OnClicked(this, &SHyperlinkWithTextHighlight::Hyperlink_OnClicked)
			.ForegroundColor(FSlateColor::UseForeground())
			.TextShapingMethod(InArgs._TextShapingMethod)
			.TextFlowDirection(InArgs._TextFlowDirection)
			[
				SAssignNew(EditableTextBlock, SInlineEditableTextBlock)
				.Style(&EditableTextBlockStyle)
				.Text(InArgs._Text)
				.HighlightText(InArgs._HighlightText)
				.OnTextCommitted(InArgs._OnTextCommitted)
				.IsSelected(InArgs._IsSelected)
			]
		);
	}

public:

	// SWidget overrides

	virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override
	{
		return FCursorReply::Cursor( EMouseCursor::Hand );
	}
#if WITH_ACCESSIBILITY
	virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override
	{
		return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleHyperlink(SharedThis(this)));
	}
#endif

	TSharedPtr<SInlineEditableTextBlock> EditableTextBlock;

protected:

	/** Invoke the OnNavigate method */
	FReply Hyperlink_OnClicked()
	{
		OnNavigate.ExecuteIfBound();

		return FReply::Handled();
	}

	/** The delegate to invoke when someone clicks the hyperlink */
	FSimpleDelegate OnNavigate;

	FInlineEditableTextBlockStyle EditableTextBlockStyle;
};
