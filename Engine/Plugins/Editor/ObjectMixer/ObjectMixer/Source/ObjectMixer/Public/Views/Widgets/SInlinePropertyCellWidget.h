// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerTreeItem.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IPropertyHandle;

struct FObjectMixerEditorListRowData;

class SInlinePropertyCellWidget : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SInlinePropertyCellWidget)
	{}

	SLATE_EVENT(TDelegate<void(const FPropertyChangedEvent&)>, OnPropertyValueChanged)

	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const FName InColumnName, const TSharedRef<ISceneOutlinerTreeItem> RowPtr);

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	FReply MakePropertyContextMenu(const FPointerEvent& MouseEvent);

protected:

	void CopyPropertyValue() const;

	void PastePropertyValue() const;

	bool CanPaste() const;

	void CopyPropertyName() const;

	TWeakPtr<ISceneOutlinerTreeItem> WeakRowPtr;
	FName ColumnName;
};
